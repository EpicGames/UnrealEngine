// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshAttributes.h"

namespace MeshAttribute
{
	const FName VertexInstance::TextureCoordinate("TextureCoordinate");
	const FName VertexInstance::Normal("Normal");
	const FName VertexInstance::Tangent("Tangent");
	const FName VertexInstance::BinormalSign("BinormalSign");
	const FName VertexInstance::Color("Color");

	const FName Edge::IsHard("IsHard");

	const FName Triangle::Normal("Normal");
	const FName Triangle::Tangent("Tangent");
	const FName Triangle::Binormal("Binormal");

	const FName Polygon::ObjectName("ObjectName");

	const FName PolygonGroup::ImportedMaterialSlotName("ImportedMaterialSlotName");

}



void FStaticMeshAttributes::Register(bool bKeepExistingAttribute)
{
	// Add basic vertex attributes

	// Add basic vertex instance attributes
	if (!MeshDescription.VertexInstanceAttributes().HasAttribute(MeshAttribute::VertexInstance::TextureCoordinate) || !bKeepExistingAttribute)
	{
		const int32 NumUVChannels = MeshDescription.GetNumUVElementChannels();
		MeshDescription.VertexInstanceAttributes().RegisterAttribute<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate, NumUVChannels, FVector2f::ZeroVector, EMeshAttributeFlags::Lerpable | EMeshAttributeFlags::Mandatory);
	}

	if (!MeshDescription.VertexInstanceAttributes().HasAttribute(MeshAttribute::VertexInstance::Normal) || !bKeepExistingAttribute)
	{
		MeshDescription.VertexInstanceAttributes().RegisterAttribute<FVector3f>(MeshAttribute::VertexInstance::Normal, 1, FVector3f::ZeroVector, EMeshAttributeFlags::AutoGenerated | EMeshAttributeFlags::Mandatory);
	}

	if (!MeshDescription.VertexInstanceAttributes().HasAttribute(MeshAttribute::VertexInstance::Tangent) || !bKeepExistingAttribute)
	{
		MeshDescription.VertexInstanceAttributes().RegisterAttribute<FVector3f>(MeshAttribute::VertexInstance::Tangent, 1, FVector3f::ZeroVector, EMeshAttributeFlags::AutoGenerated | EMeshAttributeFlags::Mandatory);
	}

	if (!MeshDescription.VertexInstanceAttributes().HasAttribute(MeshAttribute::VertexInstance::BinormalSign) || !bKeepExistingAttribute)
	{
		MeshDescription.VertexInstanceAttributes().RegisterAttribute<float>(MeshAttribute::VertexInstance::BinormalSign, 1, 0.0f, EMeshAttributeFlags::AutoGenerated | EMeshAttributeFlags::Mandatory);
	}

	if (!MeshDescription.VertexInstanceAttributes().HasAttribute(MeshAttribute::VertexInstance::Color) || !bKeepExistingAttribute)
	{
		MeshDescription.VertexInstanceAttributes().RegisterAttribute<FVector4f>(MeshAttribute::VertexInstance::Color, 1, FVector4f(1.0f, 1.0f, 1.0f, 1.0f), EMeshAttributeFlags::Lerpable | EMeshAttributeFlags::Mandatory);
	}

	// Add basic edge attributes
	if (!MeshDescription.EdgeAttributes().HasAttribute(MeshAttribute::Edge::IsHard) || !bKeepExistingAttribute)
	{
		MeshDescription.EdgeAttributes().RegisterAttribute<bool>(MeshAttribute::Edge::IsHard, 1, false, EMeshAttributeFlags::Mandatory);
	}

	// Add basic polygon attributes

	// Add basic polygon group attributes
	if (!MeshDescription.PolygonGroupAttributes().HasAttribute(MeshAttribute::PolygonGroup::ImportedMaterialSlotName) || !bKeepExistingAttribute)
	{
		MeshDescription.PolygonGroupAttributes().RegisterAttribute<FName>(MeshAttribute::PolygonGroup::ImportedMaterialSlotName, 1, NAME_None, EMeshAttributeFlags::Mandatory); //The unique key to match the mesh material slot
	}

	// Call super class
	FMeshAttributes::Register(bKeepExistingAttribute);
}


void FStaticMeshAttributes::RegisterTriangleNormalAndTangentAttributes()
{
	MeshDescription.TriangleAttributes().RegisterAttribute<FVector3f>(MeshAttribute::Triangle::Normal, 1, FVector3f::ZeroVector, EMeshAttributeFlags::Transient);
	MeshDescription.TriangleAttributes().RegisterAttribute<FVector3f>(MeshAttribute::Triangle::Tangent, 1, FVector3f::ZeroVector, EMeshAttributeFlags::Transient);
	MeshDescription.TriangleAttributes().RegisterAttribute<FVector3f>(MeshAttribute::Triangle::Binormal, 1, FVector3f::ZeroVector, EMeshAttributeFlags::Transient);
}

void FStaticMeshAttributes::UnregisterTriangleNormalAndTangentAttributes()
{
	MeshDescription.TriangleAttributes().UnregisterAttribute(MeshAttribute::Triangle::Normal);
	MeshDescription.TriangleAttributes().UnregisterAttribute(MeshAttribute::Triangle::Tangent);
	MeshDescription.TriangleAttributes().UnregisterAttribute(MeshAttribute::Triangle::Binormal);
}


void FStaticMeshAttributes::RegisterPolygonObjectNameAttribute()
{
	if (!MeshDescription.PolygonAttributes().HasAttribute(MeshAttribute::Polygon::ObjectName))
	{
		MeshDescription.PolygonAttributes().RegisterAttribute<FName>(MeshAttribute::Polygon::ObjectName, 1, NAME_None, EMeshAttributeFlags::AutoGenerated);
	}
}
