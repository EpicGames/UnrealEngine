// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshDescription.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StaticMeshDescription)


void UStaticMeshDescription::RegisterAttributes()
{
	RequiredAttributes = MakeUnique<FStaticMeshAttributes>(GetMeshDescription());
	RequiredAttributes->Register();
}


FVector2D UStaticMeshDescription::GetVertexInstanceUV(FVertexInstanceID VertexInstanceID, int32 UVIndex) const
{
	if (!GetMeshDescription().IsVertexInstanceValid(VertexInstanceID))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetVertexInstanceUV: VertexInstanceID %d doesn't exist.", VertexInstanceID.GetValue());
		return FVector2D::ZeroVector;
	}

	if (!GetMeshDescription().VertexInstanceAttributes().HasAttribute(MeshAttribute::VertexInstance::TextureCoordinate))
	{
		UE_LOGF(LogMeshDescription, Warning, "GetVertexInstanceUV: VertexInstanceAttribute TextureCoordinate doesn't exist.");
		return FVector2D::ZeroVector;
	}

	return (FVector2D)GetMeshDescription().VertexInstanceAttributes().GetAttribute<FVector2f>(VertexInstanceID, MeshAttribute::VertexInstance::TextureCoordinate, UVIndex);
}


void UStaticMeshDescription::SetVertexInstanceUV(FVertexInstanceID VertexInstanceID, FVector2D UV, int32 UVIndex)
{
	if (!GetMeshDescription().IsVertexInstanceValid(VertexInstanceID))
	{
		UE_LOGF(LogMeshDescription, Warning, "SetVertexInstanceUV: VertexInstanceID %d doesn't exist.", VertexInstanceID.GetValue());
		return;
	}

	if (!GetMeshDescription().VertexInstanceAttributes().HasAttribute(MeshAttribute::VertexInstance::TextureCoordinate))
	{
		UE_LOGF(LogMeshDescription, Warning, "SetVertexInstanceUV: VertexInstanceAttribute TextureCoordinate doesn't exist.");
		return;
	}

	GetMeshDescription().VertexInstanceAttributes().SetAttribute(VertexInstanceID, MeshAttribute::VertexInstance::TextureCoordinate, UVIndex, FVector2f(UV));
}


void UStaticMeshDescription::SetPolygonGroupMaterialSlotName(FPolygonGroupID PolygonGroupID, const FName& SlotName)
{
	if (!GetMeshDescription().IsPolygonGroupValid(PolygonGroupID))
	{
		UE_LOGF(LogMeshDescription, Warning, "SetPolygonGroupMaterialSlotName: PolygonGroupID %d doesn't exist.", PolygonGroupID.GetValue());
		return;
	}

	if (!GetMeshDescription().PolygonGroupAttributes().HasAttribute(MeshAttribute::PolygonGroup::ImportedMaterialSlotName))
	{
		UE_LOGF(LogMeshDescription, Warning, "SetPolygonGroupMaterialSlotName: PolygonGroupAttribute ImportedMaterialSlotName doesn't exist.");
		return;
	}

	GetMeshDescription().PolygonGroupAttributes().SetAttribute(PolygonGroupID, MeshAttribute::PolygonGroup::ImportedMaterialSlotName, 0, SlotName);
}


void UStaticMeshDescription::CreateCube(FVector Center, FVector HalfExtents, FPolygonGroupID PolygonGroup,
										FPolygonID& PolygonID_PlusX,
										FPolygonID& PolygonID_MinusX,
										FPolygonID& PolygonID_PlusY, 
										FPolygonID& PolygonID_MinusY,
										FPolygonID& PolygonID_PlusZ,
										FPolygonID& PolygonID_MinusZ)
{
	FMeshDescription& MeshDescription = GetMeshDescription();
	TVertexAttributesRef<FVector3f> Positions = GetVertexPositions();

	FVertexID VertexIDs[8];

	MeshDescription.ReserveNewVertices(8);
	for (int32 Index = 0; Index < 8; ++Index)
	{
		VertexIDs[Index] = MeshDescription.CreateVertex();
	}

	const FVector3f CenterPlusHalfExtent(Center + HalfExtents);	//LWC_TODO: Precision loss

	Positions[VertexIDs[0]] = CenterPlusHalfExtent * FVector3f( 1.0f, -1.0f,  1.0f);
	Positions[VertexIDs[1]] = CenterPlusHalfExtent * FVector3f( 1.0f,  1.0f,  1.0f);
	Positions[VertexIDs[2]] = CenterPlusHalfExtent * FVector3f(-1.0f,  1.0f,  1.0f);
	Positions[VertexIDs[3]] = CenterPlusHalfExtent * FVector3f(-1.0f, -1.0f,  1.0f);
	Positions[VertexIDs[4]] = CenterPlusHalfExtent * FVector3f(-1.0f,  1.0f, -1.0f);
	Positions[VertexIDs[5]] = CenterPlusHalfExtent * FVector3f(-1.0f, -1.0f, -1.0f);
	Positions[VertexIDs[6]] = CenterPlusHalfExtent * FVector3f( 1.0f, -1.0f, -1.0f);
	Positions[VertexIDs[7]] = CenterPlusHalfExtent * FVector3f( 1.0f,  1.0f, -1.0f);

	auto MakePolygon = [this, &MeshDescription, &VertexIDs, PolygonGroup](int32 P0, int32 P1, int32 P2, int32 P3) -> FPolygonID
	{
		FVertexInstanceID VertexInstanceIDs[4];
		VertexInstanceIDs[0] = MeshDescription.CreateVertexInstance(VertexIDs[P0]);
		VertexInstanceIDs[1] = MeshDescription.CreateVertexInstance(VertexIDs[P1]);
		VertexInstanceIDs[2] = MeshDescription.CreateVertexInstance(VertexIDs[P2]);
		VertexInstanceIDs[3] = MeshDescription.CreateVertexInstance(VertexIDs[P3]);

		TVertexInstanceAttributesRef<FVector2f> UVs = GetVertexInstanceUVs();
		UVs[VertexInstanceIDs[0]] = FVector2f(0.0f, 0.0f);
		UVs[VertexInstanceIDs[1]] = FVector2f(1.0f, 0.0f);
		UVs[VertexInstanceIDs[2]] = FVector2f(1.0f, 1.0f);
		UVs[VertexInstanceIDs[3]] = FVector2f(0.0f, 1.0f);

		TVertexInstanceAttributesRef<FVector4f> Colors = GetVertexInstanceColors();
		Colors[VertexInstanceIDs[0]] = (FVector4f)FLinearColor::Red;
		Colors[VertexInstanceIDs[1]] = (FVector4f)FLinearColor::Green;
		Colors[VertexInstanceIDs[2]] = (FVector4f)FLinearColor::Blue;
		Colors[VertexInstanceIDs[3]] = (FVector4f)FLinearColor::White;

		TArray<FEdgeID> EdgeIDs;
		EdgeIDs.Reserve(4);

		FPolygonID PolygonID = MeshDescription.CreatePolygon(PolygonGroup, VertexInstanceIDs, &EdgeIDs);

		for (FEdgeID EdgeID : EdgeIDs)
		{
			GetEdgeHardnesses()[EdgeID] = true;
		}

		return PolygonID;
	};

	PolygonID_PlusX = MakePolygon(0, 1, 7, 6);
	PolygonID_MinusX = MakePolygon(2, 3, 5, 4);
	PolygonID_PlusY = MakePolygon(1, 2, 4, 7);
	PolygonID_MinusY = MakePolygon(3, 0, 6, 5);
	PolygonID_PlusZ = MakePolygon(1, 0, 3, 2);
	PolygonID_MinusZ = MakePolygon(6, 7, 4, 5);

	FStaticMeshOperations::ComputeTriangleTangentsAndNormals(MeshDescription);
	FStaticMeshOperations::ComputeTangentsAndNormals(MeshDescription, EComputeNTBsFlags::Normals | EComputeNTBsFlags::Tangents);
}



