// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class FStaticMeshUtilitiesModule : public IModuleInterface
{
public:

	FStaticMeshUtilitiesModule()
	{
	}

	virtual void StartupModule() override
	{
		FModuleManager::Get().LoadModule("GeometryProcessingInterfaces");
	}

	virtual void ShutdownModule() override
	{
		// Unregister any modular features here
	}

private:

};

IMPLEMENT_MODULE(FStaticMeshUtilitiesModule, StaticMeshDescription);
