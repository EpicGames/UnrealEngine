// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticMeshOperations.h"

#include "StaticMeshAttributes.h"
#include "UVMapSettings.h"

#include "Async/ParallelFor.h"
#include "LayoutUV.h"
#include "MeshUtilitiesCommon.h"
#include "Misc/SecureHash.h"
#include "OverlappingCorners.h"
#include "RawMesh.h"
#include "Modules/ModuleManager.h"

#if WITH_EDITOR
#include "IGeometryProcessingInterfacesModule.h"
#include "GeometryProcessingInterfaces/MeshAutoUV.h"
#endif

#if WITH_MIKKTSPACE
#include "mikktspace.h"
#endif //WITH_MIKKTSPACE

DEFINE_LOG_CATEGORY(LogStaticMeshOperations);

#define LOCTEXT_NAMESPACE "StaticMeshOperations"



static TTuple<FVector3f, FVector3f, FVector3f> GetTriangleTangentsAndNormalsWithUV(float ComparisonThreshold, TArrayView<const FVector3f> VertexPositions, TArrayView<const FVector2D> VertexUVs)
{
	// GetSafeNormal compare the squareSum to the tolerance.
	const float SquareComparisonThreshold = FMath::Max(ComparisonThreshold * ComparisonThreshold, MIN_flt);

	const FVector3f Position0 = VertexPositions[0];
	// If the positions deltas are too small, we get a zero vector out.
	const FVector3f DPosition1 = VertexPositions[1] - Position0;
	const FVector3f DPosition2 = VertexPositions[2] - Position0;

	const FVector2f UV0 = FVector2f(VertexUVs[0]);
	const FVector2f DUV1 = FVector2f(VertexUVs[1]) - UV0;
	const FVector2f DUV2 = FVector2f(VertexUVs[2]) - UV0;

	// We have a left-handed coordinate system, but a counter-clockwise winding order
	// Hence normal calculation has to take the triangle vectors cross product in reverse.
	// If we got a zero vector out above, then this is also zero
	FVector3f Normal = FVector3f::CrossProduct(DPosition2, DPosition1).GetSafeNormal(SquareComparisonThreshold);
	if (!Normal.IsNearlyZero(ComparisonThreshold))
	{
		FMatrix44f	ParameterToLocal(
			DPosition1,
			DPosition2,
			Position0,
			FVector3f::ZeroVector
		);

		FMatrix44f ParameterToTexture(
			FPlane4f(DUV1.X, DUV1.Y, 0, 0),
			FPlane4f(DUV2.X, DUV2.Y, 0, 0),
			FPlane4f(UV0.X, UV0.Y, 1, 0),
			FPlane4f(0, 0, 0, 1)
		);

		// Use InverseSlow to catch singular matrices.  Inverse can miss this sometimes.
		const FMatrix44f TextureToLocal = ParameterToTexture.Inverse() * ParameterToLocal;

		FVector3f Tangent = TextureToLocal.TransformVector(FVector3f(1, 0, 0)).GetSafeNormal();
		FVector3f Binormal = TextureToLocal.TransformVector(FVector3f(0, 1, 0)).GetSafeNormal();
		FVector3f::CreateOrthonormalBasis(Tangent, Binormal, Normal);

		if (Tangent.IsNearlyZero() || Tangent.ContainsNaN()
			|| Binormal.IsNearlyZero() || Binormal.ContainsNaN())
		{
			Tangent = FVector3f::ZeroVector;
			Binormal = FVector3f::ZeroVector;
		}

		if (Normal.IsNearlyZero() || Normal.ContainsNaN())
		{
			Normal = FVector3f::ZeroVector;
		}

		return MakeTuple(Normal.GetSafeNormal(), Tangent.GetSafeNormal(), Binormal.GetSafeNormal());
	}
	else
	{
		// This will force a recompute of the normals and tangents
		return MakeTuple(FVector3f::ZeroVector, FVector3f::ZeroVector, FVector3f::ZeroVector);
	}
}

// Create a normal using the triangle plane, but use the Duff & Frisvad algorithm (see Duff 2017 in JCGT) to construct a consistent tangent from a
// single vector. 
static TTuple<FVector3f, FVector3f, FVector3f> GetTriangleTangentsAndNormalsWithNoUVs(float ComparisonThreshold, TArrayView<const FVector3f> VertexPositions)
{
	// GetSafeNormal compare the squareSum to the tolerance.
	const float SquareComparisonThreshold = FMath::Max(ComparisonThreshold * ComparisonThreshold, MIN_flt);

	const FVector3f Position0 = VertexPositions[0];
	// If the positions deltas are too small, we get a zero vector out.
	const FVector3f DPosition1 = VertexPositions[1] - Position0;
	const FVector3f DPosition2 = VertexPositions[2] - Position0;

	// We have a left-handed coordinate system, but a counter-clockwise winding order
	// Hence normal calculation has to take the triangle vectors cross product in reverse.
	// If we got a zero vector out above, then this is also zero
	FVector3f Normal = FVector3f::CrossProduct(DPosition2, DPosition1).GetSafeNormal(SquareComparisonThreshold);
	if (!Normal.Normalize(ComparisonThreshold))
	{
		return MakeTuple(FVector3f::ZeroVector, FVector3f::ZeroVector, FVector3f::ZeroVector);
	}

	FVector3f Tangent, Binormal;
	if (Normal.Z < 0.0f)
	{
		const float A = 1.0f / (1.0f - Normal.Z);
		const float B = Normal.X * Normal.Y * A;

		Tangent = FVector3f(1.0f - Normal.X * Normal.X * A, -B, Normal.X);
		Binormal = FVector3f(B, Normal.Y * Normal.Y * A - 1.0f, -Normal.Y);
	}
	else
	{
		const float A = 1.0f / (1.0f + Normal.Z);
		const float B = -Normal.X * Normal.Y * A;

		Tangent = FVector3f(1.0f - Normal.X * Normal.X * A, B, -Normal.X);
		Binormal = FVector3f(B, 1.0f - Normal.Y * Normal.Y * A, -Normal.Y);
	}

	// The above algorithm guarantees orthogonality and normalization of the tangent & binormal if the normal vector is already normalized.
	return MakeTuple(Normal, Tangent, Binormal);
}


void FStaticMeshOperations::ComputeTriangleTangentsAndNormals(FMeshDescription& MeshDescription, float ComparisonThreshold, const TCHAR* DebugName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshOperations::ComputeTriangleTangentsAndNormals_Selection);

	FStaticMeshAttributes Attributes(MeshDescription);
	Attributes.RegisterTriangleNormalAndTangentAttributes();

	// Check that the mesh description is compact
	const int32 NumTriangles = MeshDescription.Triangles().Num();
	if (MeshDescription.NeedsCompact())
	{
		FElementIDRemappings Remappings;
		MeshDescription.Compact(Remappings);
	}

	// Split work in batch to reduce call overhead
	const int32 BatchSize = 8 * 1024;
	const int32 BatchCount = (NumTriangles + BatchSize - 1) / BatchSize;

	ParallelFor( TEXT("ComputeTriangleTangentsAndNormals.PF"), BatchCount,1,
		[BatchSize, ComparisonThreshold, NumTriangles, &Attributes, DebugName](int32 BatchIndex)
		{
			TArrayView<const FVector3f> VertexPositions = Attributes.GetVertexPositions().GetRawArray();
			TArrayView<const FVector2f> VertexUVs;
			TArrayView<const FVertexID> TriangleVertexIDs = Attributes.GetTriangleVertexIndices().GetRawArray();
			TArrayView<const FVertexInstanceID> TriangleVertexInstanceIDs = Attributes.GetTriangleVertexInstanceIndices().GetRawArray();

			TArrayView<FVector3f> TriangleNormals = Attributes.GetTriangleNormals().GetRawArray();
			TArrayView<FVector3f> TriangleTangents = Attributes.GetTriangleTangents().GetRawArray();
			TArrayView<FVector3f> TriangleBinormals = Attributes.GetTriangleBinormals().GetRawArray();

			if (Attributes.GetVertexInstanceUVs().IsValid() && Attributes.GetVertexInstanceUVs().GetNumChannels() > 0)
			{
				VertexUVs = Attributes.GetVertexInstanceUVs().GetRawArray(0);
			}

			int32 StartIndex = BatchIndex * BatchSize;
			int32 TriIndex = StartIndex * 3;
			int32 EndIndex = FMath::Min(StartIndex + BatchSize, NumTriangles);
			for (; StartIndex < EndIndex; ++StartIndex, TriIndex += 3)
			{
				if (!TriangleNormals[StartIndex].IsNearlyZero())
				{
					// Bypass normal calculation if it's already done
					continue;
				}

				if (!TriangleVertexIDs.IsValidIndex(TriIndex + 2))
				{
					UE_LOGF(LogStaticMeshOperations, Warning, "Static Mesh %ls has invalid TriangleVertexIDs length! TriIndex %d. Skipping",
						DebugName ? DebugName : TEXT("UnknownMesh"), TriIndex + 2);
					TriangleNormals[StartIndex] = FVector3f(1, 0, 0);
					TriangleTangents[StartIndex] = FVector3f(0, 1, 0);
					TriangleBinormals[StartIndex] = FVector3f(0, 0, 1);
					continue;
				}

				if (!VertexPositions.IsValidIndex(TriangleVertexIDs[TriIndex]) ||
					!VertexPositions.IsValidIndex(TriangleVertexIDs[TriIndex + 1]) ||
					!VertexPositions.IsValidIndex(TriangleVertexIDs[TriIndex + 2])
					)
				{
					UE_LOGF(LogStaticMeshOperations, Warning, "Static Mesh %ls has invalid vertex ids! StartIndex %d. VertId0=%d, VertId1=%d, VertId2=%d. Skipping",
						DebugName ? DebugName : TEXT("UnknownMesh"), StartIndex, TriangleVertexIDs[TriIndex].GetValue(), TriangleVertexIDs[TriIndex + 1].GetValue(), TriangleVertexIDs[TriIndex + 2].GetValue());
					TriangleNormals[StartIndex] = FVector3f(1, 0, 0);
					TriangleTangents[StartIndex] = FVector3f(0, 1, 0);
					TriangleBinormals[StartIndex] = FVector3f(0, 0, 1);
					continue;
				}

				FVector3f TriangleVertexPositions[3] =
				{
					VertexPositions[TriangleVertexIDs[TriIndex]],
					VertexPositions[TriangleVertexIDs[TriIndex + 1]],
					VertexPositions[TriangleVertexIDs[TriIndex + 2]]
				};

				if (TriangleVertexPositions[0].ContainsNaN() ||
					TriangleVertexPositions[1].ContainsNaN() ||
					TriangleVertexPositions[2].ContainsNaN())
				{
					UE_CLOGF(DebugName != nullptr, LogStaticMeshOperations, Warning, "Static Mesh %ls has NaNs in it's vertex positions! Triangle index %d -- using identity for tangent basis.", DebugName, StartIndex);
					TriangleNormals[StartIndex] = FVector3f(1, 0, 0);
					TriangleTangents[StartIndex] = FVector3f(0, 1, 0);
					TriangleBinormals[StartIndex] = FVector3f(0, 0, 1);
					continue;
				}

				TTuple<FVector3f, FVector3f, FVector3f> Result;
				if (!VertexUVs.IsEmpty())
				{
					if (!TriangleVertexInstanceIDs.IsValidIndex(TriIndex + 2))
					{
						UE_LOGF(LogStaticMeshOperations, Warning, "Static Mesh %ls has invalid TriangleVertexInstanceIDs length! TriIndex %d. Skipping",
							DebugName ? DebugName : TEXT("UnknownMesh"), TriIndex + 2);
						TriangleNormals[StartIndex] = FVector3f(1, 0, 0);
						TriangleTangents[StartIndex] = FVector3f(0, 1, 0);
						TriangleBinormals[StartIndex] = FVector3f(0, 0, 1);
						continue;
					}

					if (!VertexUVs.IsValidIndex(TriangleVertexInstanceIDs[TriIndex]) ||
						!VertexUVs.IsValidIndex(TriangleVertexInstanceIDs[TriIndex + 1]) ||
						!VertexUVs.IsValidIndex(TriangleVertexInstanceIDs[TriIndex + 2])
						)
					{
						UE_LOGF(LogStaticMeshOperations, Warning, "Static Mesh %ls has invalid vertex instance ids! Triangle index %d. InstanceId0=%d, InstanceId1=%d, InstanceId2=%d. Skipping",
							DebugName ? DebugName : TEXT("UnknownMesh"), StartIndex, TriangleVertexInstanceIDs[TriIndex].GetValue(), TriangleVertexInstanceIDs[TriIndex + 1].GetValue(), TriangleVertexInstanceIDs[TriIndex + 2].GetValue());
						TriangleNormals[StartIndex] = FVector3f(1, 0, 0);
						TriangleTangents[StartIndex] = FVector3f(0, 1, 0);
						TriangleBinormals[StartIndex] = FVector3f(0, 0, 1);
						continue;
					}

					FVector2D TriangleUVs[3] =
					{
						FVector2D(VertexUVs[TriangleVertexInstanceIDs[TriIndex]]),
						FVector2D(VertexUVs[TriangleVertexInstanceIDs[TriIndex + 1]]),
						FVector2D(VertexUVs[TriangleVertexInstanceIDs[TriIndex + 2]])
					};

					if (TriangleUVs[0].ContainsNaN() ||
						TriangleUVs[1].ContainsNaN() ||
						TriangleUVs[2].ContainsNaN())
					{
						UE_CLOGF(DebugName != nullptr, LogStaticMeshOperations, Warning, "Static Mesh %ls has NaNs in it's vertex uvs! Triangle index %d -- using identity for tangent basis.", DebugName, StartIndex);
						TriangleNormals[StartIndex] = FVector3f(1, 0, 0);
						TriangleTangents[StartIndex] = FVector3f(0, 1, 0);
						TriangleBinormals[StartIndex] = FVector3f(0, 0, 1);
						continue;
					}

					Result = GetTriangleTangentsAndNormalsWithUV(ComparisonThreshold, TriangleVertexPositions, TriangleUVs);
				}
				else
				{
					Result = GetTriangleTangentsAndNormalsWithNoUVs(ComparisonThreshold, TriangleVertexPositions);
				}
				TriangleNormals[StartIndex] = Result.Get<0>();
				TriangleTangents[StartIndex] = Result.Get<1>();
				TriangleBinormals[StartIndex] = Result.Get<2>();
		}
		}
	);
}


void FStaticMeshOperations::DetermineEdgeHardnessesFromVertexInstanceNormals(FMeshDescription& MeshDescription, float Tolerance)
{
	FStaticMeshAttributes Attributes(MeshDescription);

	TVertexInstanceAttributesRef<const FVector3f> VertexNormals = Attributes.GetVertexInstanceNormals();
	TEdgeAttributesRef<bool> EdgeHardnesses = Attributes.GetEdgeHardnesses();

	// Holds unique vertex instance IDs for a given edge vertex
	TArray<FVertexInstanceID> UniqueVertexInstanceIDs;

	for (const FEdgeID EdgeID : MeshDescription.Edges().GetElementIDs())
	{
		// Get list of polygons connected to this edge
		TArray<FPolygonID, TInlineAllocator<2>> ConnectedPolygonIDs = MeshDescription.GetEdgeConnectedPolygons<TInlineAllocator<2>>(EdgeID);
		if (ConnectedPolygonIDs.Num() == 0)
		{
			// What does it mean if an edge has no connected polygons? For now we just skip it
			continue;
		}

		// Assume by default that the edge is soft - but as soon as any vertex instance belonging to a connected polygon
		// has a distinct normal from the others (within the given tolerance), we mark it as hard.
		// The exception is if an edge has exactly one connected polygon: in this case we automatically deem it a hard edge.
		bool bEdgeIsHard = (ConnectedPolygonIDs.Num() == 1);

		// Examine vertices on each end of the edge, if we haven't yet identified it as 'hard'
		for (int32 VertexIndex = 0; !bEdgeIsHard && VertexIndex < 2; ++VertexIndex)
		{
			const FVertexID VertexID = MeshDescription.GetEdgeVertex(EdgeID, VertexIndex);

			const int32 ReservedElements = 4;
			UniqueVertexInstanceIDs.Reset(ReservedElements);

			// Get a list of all vertex instances for this vertex which form part of any polygon connected to the edge
			for (const FVertexInstanceID& VertexInstanceID : MeshDescription.GetVertexVertexInstanceIDs(VertexID))
			{
				for (const FPolygonID& PolygonID : MeshDescription.GetVertexInstanceConnectedPolygons<TInlineAllocator<8>>(VertexInstanceID))
				{
					if (ConnectedPolygonIDs.Contains(PolygonID))
					{
						UniqueVertexInstanceIDs.AddUnique(VertexInstanceID);
						break;
					}
				}
			}
			check(UniqueVertexInstanceIDs.Num() > 0);

			// First unique vertex instance is used as a reference against which the others are compared.
			// (not a perfect approach: really the 'median' should be used as a reference)
			const FVector3f ReferenceNormal = VertexNormals[UniqueVertexInstanceIDs[0]];
			for (int32 Index = 1; Index < UniqueVertexInstanceIDs.Num(); ++Index)
			{
				if (!VertexNormals[UniqueVertexInstanceIDs[Index]].Equals(ReferenceNormal, Tolerance))
				{
					bEdgeIsHard = true;
					break;
				}
			}
		}

		EdgeHardnesses[EdgeID] = bEdgeIsHard;
	}
}


//////////////////

struct FVertexInfo
{
	FVertexInfo()
	{
		TriangleID = INDEX_NONE;
		VertexInstanceID = INDEX_NONE;
		UVs = FVector2f(0.0f, 0.0f);
	}

	FTriangleID TriangleID;
	FVertexInstanceID VertexInstanceID;
	FVector2f UVs;
	//Most of the time a edge has two triangles
	TArray<FEdgeID, TInlineAllocator<2>> EdgeIDs;
};

/** Helper struct for building acceleration structures. */
namespace MeshDescriptionOperationNamespace
{
	struct FIndexAndZ
	{
		float Z;
		int32 Index;
		const FVector3f *OriginalVector;

		/** Default constructor. */
		FIndexAndZ() {}

		/** Initialization constructor. */
		FIndexAndZ(int32 InIndex, const FVector3f& V)
		{
			Z = 0.30f * V.X + 0.33f * V.Y + 0.37f * V.Z;
			Index = InIndex;
			OriginalVector = &V;
		}
	};
	/** Sorting function for vertex Z/index pairs. */
	struct FCompareIndexAndZ
	{
		FORCEINLINE bool operator()(const FIndexAndZ& A, const FIndexAndZ& B) const { return A.Z < B.Z; }
	};
}

void FStaticMeshOperations::ConvertHardEdgesToSmoothGroup(const FMeshDescription& SourceMeshDescription, TArray<uint32>& FaceSmoothingMasks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshOperations::ConvertHardEdgesToSmoothGroup);

	TMap<FPolygonID, uint32> PolygonSmoothGroup;
	PolygonSmoothGroup.Reserve(SourceMeshDescription.Polygons().GetArraySize());
	TArray<bool> ConsumedPolygons;
	ConsumedPolygons.AddZeroed(SourceMeshDescription.Polygons().GetArraySize());

	TMap < FPolygonID, uint32> PolygonAvoidances;

	TEdgeAttributesConstRef<bool> EdgeHardnesses = SourceMeshDescription.EdgeAttributes().GetAttributesRef<bool>(MeshAttribute::Edge::IsHard);
	int32 TriangleCount = 0;
	TArray<FPolygonID> SoftEdgeNeigbors;
	TArray<FEdgeID> PolygonEdges;
	TArray<FPolygonID> EdgeConnectedPolygons;
	TArray<FPolygonID> ConnectedPolygons;
	TArray<FPolygonID> LastConnectedPolygons;

	for (const FPolygonID PolygonID : SourceMeshDescription.Polygons().GetElementIDs())
	{
		TriangleCount += SourceMeshDescription.GetPolygonTriangles(PolygonID).Num();
		if (ConsumedPolygons[PolygonID.GetValue()])
		{
			continue;
		}

		ConnectedPolygons.Reset();
		LastConnectedPolygons.Reset();
		ConnectedPolygons.Add(PolygonID);
		LastConnectedPolygons.Add(INDEX_NONE);
		while (ConnectedPolygons.Num() > 0)
		{
			check(LastConnectedPolygons.Num() == ConnectedPolygons.Num());
			FPolygonID LastPolygonID = LastConnectedPolygons.Pop(EAllowShrinking::No);
			FPolygonID CurrentPolygonID = ConnectedPolygons.Pop(EAllowShrinking::No);
			if (ConsumedPolygons[CurrentPolygonID.GetValue()])
			{
				continue;
			}
			SoftEdgeNeigbors.Reset();
			uint32& SmoothGroup = PolygonSmoothGroup.FindOrAdd(CurrentPolygonID);
			uint32 AvoidSmoothGroup = 0;
			uint32 NeighborSmoothGroup = 0;
			const uint32 LastSmoothGroupValue = (LastPolygonID == INDEX_NONE) ? 0 : PolygonSmoothGroup[LastPolygonID];
			PolygonEdges.Reset();
			SourceMeshDescription.GetPolygonPerimeterEdges(CurrentPolygonID, PolygonEdges);
			for (const FEdgeID& EdgeID : PolygonEdges)
			{
				bool bIsHardEdge = EdgeHardnesses[EdgeID];
				EdgeConnectedPolygons.Reset();
				SourceMeshDescription.GetEdgeConnectedPolygons(EdgeID, EdgeConnectedPolygons);
				for (const FPolygonID& EdgePolygonID : EdgeConnectedPolygons)
				{
					if (EdgePolygonID == CurrentPolygonID)
					{
						continue;
					}
					uint32 SmoothValue = 0;
					if (PolygonSmoothGroup.Contains(EdgePolygonID))
					{
						SmoothValue = PolygonSmoothGroup[EdgePolygonID];
					}

					if (bIsHardEdge) //Hard Edge
					{
						AvoidSmoothGroup |= SmoothValue;
					}
					else
					{
						NeighborSmoothGroup |= SmoothValue;
						//Put all none hard edge polygon in the next iteration
						if (!ConsumedPolygons[EdgePolygonID.GetValue()])
						{
							ConnectedPolygons.Add(EdgePolygonID);
							LastConnectedPolygons.Add(CurrentPolygonID);
						}
						else
						{
							SoftEdgeNeigbors.Add(EdgePolygonID);
						}
					}
				}
			}

			if (AvoidSmoothGroup != 0)
			{
				PolygonAvoidances.FindOrAdd(CurrentPolygonID) = AvoidSmoothGroup;
				//find neighbor avoidance
				for (FPolygonID& NeighborID : SoftEdgeNeigbors)
				{
					if (!PolygonAvoidances.Contains(NeighborID))
					{
						continue;
					}
					AvoidSmoothGroup |= PolygonAvoidances[NeighborID];
				}
				uint32 NewSmoothGroup = 1;
				while ((NewSmoothGroup & AvoidSmoothGroup) != 0 && NewSmoothGroup < MAX_uint32)
				{
					//Shift the smooth group
					NewSmoothGroup = NewSmoothGroup << 1;
				}
				SmoothGroup = NewSmoothGroup;
				//Apply to all neighboard
				for (FPolygonID& NeighborID : SoftEdgeNeigbors)
				{
					PolygonSmoothGroup[NeighborID] |= NewSmoothGroup;
				}
			}
			else if (NeighborSmoothGroup != 0)
			{
				SmoothGroup |= LastSmoothGroupValue | NeighborSmoothGroup;
			}
			else
			{
				SmoothGroup = 1;
			}
			ConsumedPolygons[CurrentPolygonID.GetValue()] = true;
		}
	}
	//Set the smooth group in the FaceSmoothingMasks parameter
	check(FaceSmoothingMasks.Num() == TriangleCount);
	int32 TriangleIndex = 0;
	for (const FPolygonID PolygonID : SourceMeshDescription.Polygons().GetElementIDs())
	{
		const uint32 PolygonSmoothValue = PolygonSmoothGroup[PolygonID];
		for (int32 i = 0, Num = SourceMeshDescription.GetPolygonTriangles(PolygonID).Num(); i < Num; ++i)
		{
			FaceSmoothingMasks[TriangleIndex++] = PolygonSmoothValue;
		}
	}
}

void FStaticMeshOperations::ConvertSmoothGroupToHardEdges(const TArray<uint32>& FaceSmoothingMasks, FMeshDescription& DestinationMeshDescription)
{
	TEdgeAttributesRef<bool> EdgeHardnesses = DestinationMeshDescription.EdgeAttributes().GetAttributesRef<bool>(MeshAttribute::Edge::IsHard);

	TArray<bool> ConsumedPolygons;
	ConsumedPolygons.AddZeroed(DestinationMeshDescription.Polygons().Num());
	for (const FPolygonID PolygonID : DestinationMeshDescription.Polygons().GetElementIDs())
	{
		if (ConsumedPolygons[PolygonID.GetValue()])
		{
			continue;
		}
		TArray<FPolygonID> ConnectedPolygons;
		ConnectedPolygons.Add(PolygonID);
		while (ConnectedPolygons.Num() > 0)
		{
			FPolygonID CurrentPolygonID = ConnectedPolygons.Pop(EAllowShrinking::No);
			int32 CurrentPolygonIDValue = CurrentPolygonID.GetValue();
			check(FaceSmoothingMasks.IsValidIndex(CurrentPolygonIDValue));
			const uint32 ReferenceSmoothGroup = FaceSmoothingMasks[CurrentPolygonIDValue];
			TArray<FEdgeID> PolygonEdges;
			DestinationMeshDescription.GetPolygonPerimeterEdges(CurrentPolygonID, PolygonEdges);
			for (const FEdgeID& EdgeID : PolygonEdges)
			{
				const bool bIsHardEdge = EdgeHardnesses[EdgeID];
				if (bIsHardEdge)
				{
					continue;
				}
				const TArray<FPolygonID>& EdgeConnectedPolygons = DestinationMeshDescription.GetEdgeConnectedPolygons(EdgeID);
				for (const FPolygonID& EdgePolygonID : EdgeConnectedPolygons)
				{
					int32 EdgePolygonIDValue = EdgePolygonID.GetValue();
					if (EdgePolygonID == CurrentPolygonID || ConsumedPolygons[EdgePolygonIDValue])
					{
						continue;
					}
					check(FaceSmoothingMasks.IsValidIndex(EdgePolygonIDValue));
					const uint32 TestSmoothGroup = FaceSmoothingMasks[EdgePolygonIDValue];
					if ((TestSmoothGroup & ReferenceSmoothGroup) == 0)
					{
						EdgeHardnesses[EdgeID] = true;
						break;
					}
					else
					{
						ConnectedPolygons.Add(EdgePolygonID);
					}
				}
			}
			ConsumedPolygons[CurrentPolygonID.GetValue()] = true;
		}
	}
}

void FStaticMeshOperations::ConvertToRawMesh(const FMeshDescription& SourceMeshDescription, FRawMesh& DestinationRawMesh, const TMap<FName, int32>& MaterialMap)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshOperations::ConvertToRawMesh);

	DestinationRawMesh.Empty();

	//Gather all array data
	FStaticMeshConstAttributes Attributes(SourceMeshDescription);
	TVertexAttributesConstRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
	TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesConstRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesConstRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
	TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
	TPolygonGroupAttributesConstRef<FName> PolygonGroupMaterialSlotName = Attributes.GetPolygonGroupMaterialSlotNames();

	DestinationRawMesh.VertexPositions.AddZeroed(SourceMeshDescription.Vertices().Num());
	TArray<int32> RemapVerts;
	RemapVerts.AddZeroed(SourceMeshDescription.Vertices().GetArraySize());
	int32 VertexIndex = 0;
	for (const FVertexID VertexID : SourceMeshDescription.Vertices().GetElementIDs())
	{
		DestinationRawMesh.VertexPositions[VertexIndex] = VertexPositions[VertexID];
		RemapVerts[VertexID.GetValue()] = VertexIndex;
		++VertexIndex;
	}

	int32 TriangleNumber = SourceMeshDescription.Triangles().Num();
	DestinationRawMesh.FaceMaterialIndices.AddZeroed(TriangleNumber);
	DestinationRawMesh.FaceSmoothingMasks.AddZeroed(TriangleNumber);

	bool bHasVertexColor = HasVertexColor(SourceMeshDescription);

	int32 WedgeIndexNumber = TriangleNumber * 3;
	if (bHasVertexColor)
	{
		DestinationRawMesh.WedgeColors.AddZeroed(WedgeIndexNumber);
	}
	DestinationRawMesh.WedgeIndices.AddZeroed(WedgeIndexNumber);
	DestinationRawMesh.WedgeTangentX.AddZeroed(WedgeIndexNumber);
	DestinationRawMesh.WedgeTangentY.AddZeroed(WedgeIndexNumber);
	DestinationRawMesh.WedgeTangentZ.AddZeroed(WedgeIndexNumber);
	int32 ExistingUVCount = VertexInstanceUVs.GetNumChannels();
	for (int32 UVIndex = 0; UVIndex < ExistingUVCount; ++UVIndex)
	{
		DestinationRawMesh.WedgeTexCoords[UVIndex].AddZeroed(WedgeIndexNumber);
	}

	int32 TriangleIndex = 0;
	int32 WedgeIndex = 0;
	for (const FPolygonID PolygonID : SourceMeshDescription.Polygons().GetElementIDs())
	{
		const FPolygonGroupID& PolygonGroupID = SourceMeshDescription.GetPolygonPolygonGroup(PolygonID);
		int32 PolygonIDValue = PolygonID.GetValue();
		TArrayView<const FTriangleID> TriangleIDs = SourceMeshDescription.GetPolygonTriangles(PolygonID);
		for (const FTriangleID& TriangleID : TriangleIDs)
		{
			if (MaterialMap.Num() > 0 && MaterialMap.Contains(PolygonGroupMaterialSlotName[PolygonGroupID]))
			{
				DestinationRawMesh.FaceMaterialIndices[TriangleIndex] = MaterialMap[PolygonGroupMaterialSlotName[PolygonGroupID]];
			}
			else
			{
				DestinationRawMesh.FaceMaterialIndices[TriangleIndex] = PolygonGroupID.GetValue();
			}
			DestinationRawMesh.FaceSmoothingMasks[TriangleIndex] = 0; //Conversion of soft/hard to smooth mask is done after the geometry is converted
			for (int32 Corner = 0; Corner < 3; ++Corner)
			{
				const FVertexInstanceID VertexInstanceID = SourceMeshDescription.GetTriangleVertexInstance(TriangleID, Corner);

				if (bHasVertexColor)
				{
					DestinationRawMesh.WedgeColors[WedgeIndex] = FLinearColor(VertexInstanceColors[VertexInstanceID]).ToFColor(true);
				}
				DestinationRawMesh.WedgeIndices[WedgeIndex] = RemapVerts[SourceMeshDescription.GetVertexInstanceVertex(VertexInstanceID).GetValue()];
				DestinationRawMesh.WedgeTangentX[WedgeIndex] = VertexInstanceTangents[VertexInstanceID];
				DestinationRawMesh.WedgeTangentY[WedgeIndex] = FVector3f::CrossProduct(VertexInstanceNormals[VertexInstanceID], VertexInstanceTangents[VertexInstanceID]).GetSafeNormal() * VertexInstanceBinormalSigns[VertexInstanceID];
				DestinationRawMesh.WedgeTangentZ[WedgeIndex] = VertexInstanceNormals[VertexInstanceID];
				for (int32 UVIndex = 0; UVIndex < ExistingUVCount; ++UVIndex)
				{
					DestinationRawMesh.WedgeTexCoords[UVIndex][WedgeIndex] = VertexInstanceUVs.Get(VertexInstanceID, UVIndex);
				}
				++WedgeIndex;
			}
			++TriangleIndex;
		}
	}
	//Convert the smoothgroup
	ConvertHardEdgesToSmoothGroup(SourceMeshDescription, DestinationRawMesh.FaceSmoothingMasks);
}

//We want to fill the FMeshDescription vertex position mesh attribute with the FRawMesh vertex position
//We will also weld the vertex position (old FRawMesh is not always welded) and construct a mapping array to match the FVertexID
void FillMeshDescriptionVertexPositionNoDuplicate(const TArray<FVector3f>& RawMeshVertexPositions, FMeshDescription& DestinationMeshDescription, TArray<FVertexID>& RemapVertexPosition)
{
	TVertexAttributesRef<FVector3f> VertexPositions = DestinationMeshDescription.GetVertexPositions();

	const int32 NumVertex = RawMeshVertexPositions.Num();

	TMap<int32, int32> TempRemapVertexPosition;
	TempRemapVertexPosition.Reserve(NumVertex);

	// Create a list of vertex Z/index pairs
	TArray<MeshDescriptionOperationNamespace::FIndexAndZ> VertIndexAndZ;
	VertIndexAndZ.Reserve(NumVertex);

	for (int32 VertexIndex = 0; VertexIndex < NumVertex; ++VertexIndex)
	{
		VertIndexAndZ.Emplace(VertexIndex, RawMeshVertexPositions[VertexIndex]);
	}

	// Sort the vertices by z value
	VertIndexAndZ.Sort(MeshDescriptionOperationNamespace::FCompareIndexAndZ());

	int32 VertexCount = 0;
	// Search for duplicates, quickly!
	for (int32 i = 0; i < VertIndexAndZ.Num(); i++)
	{
		int32 Index_i = VertIndexAndZ[i].Index;
		if (TempRemapVertexPosition.Contains(Index_i))
		{
			continue;
		}
		TempRemapVertexPosition.FindOrAdd(Index_i) = VertexCount;
		// only need to search forward, since we add pairs both ways
		for (int32 j = i + 1; j < VertIndexAndZ.Num(); j++)
		{
			if (FMath::Abs(VertIndexAndZ[j].Z - VertIndexAndZ[i].Z) > SMALL_NUMBER)
				break; // can't be any more dups

			const FVector3f& PositionA = *(VertIndexAndZ[i].OriginalVector);
			const FVector3f& PositionB = *(VertIndexAndZ[j].OriginalVector);

			if (PositionA.Equals(PositionB, SMALL_NUMBER))
			{
				TempRemapVertexPosition.FindOrAdd(VertIndexAndZ[j].Index) = VertexCount;
			}
		}
		VertexCount++;
	}

	//Make sure the vertex are added in the same order to be lossless when converting the FRawMesh
	//In case there is a duplicate even reordering it will not be lossless, but MeshDescription do not support
	//bad data like duplicated vertex position.
	RemapVertexPosition.AddUninitialized(NumVertex);
	DestinationMeshDescription.ReserveNewVertices(VertexCount);
	TArray<FVertexID> UniqueVertexDone;
	UniqueVertexDone.AddUninitialized(VertexCount);
	for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
	{
		UniqueVertexDone[VertexIndex] = INDEX_NONE;
	}
	for (int32 VertexIndex = 0; VertexIndex < NumVertex; ++VertexIndex)
	{
		int32 RealIndex = TempRemapVertexPosition[VertexIndex];
		if (UniqueVertexDone[RealIndex] != INDEX_NONE)
		{
			RemapVertexPosition[VertexIndex] = UniqueVertexDone[RealIndex];
			continue;
		}
		FVertexID VertexID = DestinationMeshDescription.CreateVertex();
		UniqueVertexDone[RealIndex] = VertexID;
		VertexPositions[VertexID] = RawMeshVertexPositions[VertexIndex];
		RemapVertexPosition[VertexIndex] = VertexID;
	}
}

//Discover degenerated triangle
bool IsTriangleDegenerated(const FRawMesh& SourceRawMesh, const TArray<FVertexID>& RemapVertexPosition, const int32 VerticeIndexBase)
{
	FVertexID VertexIDs[3];
	for (int32 Corner = 0; Corner < 3; ++Corner)
	{
		int32 VerticeIndex = VerticeIndexBase + Corner;
		VertexIDs[Corner] = RemapVertexPosition[SourceRawMesh.WedgeIndices[VerticeIndex]];
	}
	return (VertexIDs[0] == VertexIDs[1] || VertexIDs[0] == VertexIDs[2] || VertexIDs[1] == VertexIDs[2]);
}

void FStaticMeshOperations::ConvertFromRawMesh(const FRawMesh& SourceRawMesh, FMeshDescription& DestinationMeshDescription, const TMap<int32, FName>& MaterialMap, bool bSkipNormalsAndTangents, const TCHAR* DebugName)
{
	DestinationMeshDescription.Empty();

	DestinationMeshDescription.ReserveNewVertexInstances(SourceRawMesh.WedgeIndices.Num());
	DestinationMeshDescription.ReserveNewPolygons(SourceRawMesh.WedgeIndices.Num() / 3);
	//Approximately 2.5 edges per polygons
	DestinationMeshDescription.ReserveNewEdges(SourceRawMesh.WedgeIndices.Num() * 2.5f / 3);

	//Gather all array data
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = DestinationMeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Normal);
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = DestinationMeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Tangent);
	TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = DestinationMeshDescription.VertexInstanceAttributes().GetAttributesRef<float>(MeshAttribute::VertexInstance::BinormalSign);
	TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = DestinationMeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector4f>(MeshAttribute::VertexInstance::Color);
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = DestinationMeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);

	TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = DestinationMeshDescription.PolygonGroupAttributes().GetAttributesRef<FName>(MeshAttribute::PolygonGroup::ImportedMaterialSlotName);

	int32 NumTexCoords = 0;
	int32 MaxTexCoords = MAX_MESH_TEXTURE_COORDS;
	TArray<int32> TextureCoordinnateRemapIndex;
	TextureCoordinnateRemapIndex.AddZeroed(MaxTexCoords);
	for (int32 TextureCoordinnateIndex = 0; TextureCoordinnateIndex < MaxTexCoords; ++TextureCoordinnateIndex)
	{
		TextureCoordinnateRemapIndex[TextureCoordinnateIndex] = INDEX_NONE;
		if (SourceRawMesh.WedgeTexCoords[TextureCoordinnateIndex].Num() == SourceRawMesh.WedgeIndices.Num())
		{
			TextureCoordinnateRemapIndex[TextureCoordinnateIndex] = NumTexCoords;
			NumTexCoords++;
		}
	}
	VertexInstanceUVs.SetNumChannels(NumTexCoords);

	//Ensure we do not have any duplicate, We found all duplicated vertex and compact them and build a remap indice array to remap the wedgeindices
	TArray<FVertexID> RemapVertexPosition;
	FillMeshDescriptionVertexPositionNoDuplicate(SourceRawMesh.VertexPositions, DestinationMeshDescription, RemapVertexPosition);

	bool bHasColors = SourceRawMesh.WedgeColors.Num() > 0;
	bool bHasTangents = SourceRawMesh.WedgeTangentX.Num() > 0 && SourceRawMesh.WedgeTangentY.Num() > 0;
	bool bHasNormals = SourceRawMesh.WedgeTangentZ.Num() > 0;

	TArray<FPolygonGroupID> PolygonGroups;
	TMap<int32, FPolygonGroupID> MaterialIndexToPolygonGroup;

	//Create the PolygonGroups
	int32 ActualPolygonSection = 0;
	for (int32 MaterialIndex : SourceRawMesh.FaceMaterialIndices)
	{
		if (!MaterialIndexToPolygonGroup.Contains(MaterialIndex))
		{
			FPolygonGroupID PolygonGroupID(ActualPolygonSection);
			DestinationMeshDescription.CreatePolygonGroupWithID(PolygonGroupID);
			ActualPolygonSection++;
			if (MaterialMap.Contains(MaterialIndex))
			{
				PolygonGroupImportedMaterialSlotNames[PolygonGroupID] = MaterialMap[MaterialIndex];
			}
			else
			{
				PolygonGroupImportedMaterialSlotNames[PolygonGroupID] = FName(*FString::Printf(TEXT("MaterialSlot_%d"), MaterialIndex));
			}
			PolygonGroups.Add(PolygonGroupID);
			MaterialIndexToPolygonGroup.Add(MaterialIndex, PolygonGroupID);
		}
	}

	//Triangles
	int32 TriangleCount = SourceRawMesh.WedgeIndices.Num() / 3;

	// Reserve enough memory to avoid as much as possible reallocations
	DestinationMeshDescription.ReserveNewVertexInstances(SourceRawMesh.WedgeIndices.Num());
	DestinationMeshDescription.ReserveNewTriangles(TriangleCount);
	DestinationMeshDescription.ReserveNewPolygons(TriangleCount);
	DestinationMeshDescription.ReserveNewEdges(TriangleCount * 2);

	for (int32 TriangleIndex = 0; TriangleIndex < TriangleCount; ++TriangleIndex)
	{
		int32 VerticeIndexBase = TriangleIndex * 3;
		//Check if the triangle is degenerated and skip the data if its the case
		if (IsTriangleDegenerated(SourceRawMesh, RemapVertexPosition, VerticeIndexBase))
		{
			continue;
		}

		//PolygonGroup
		FPolygonGroupID PolygonGroupID = INDEX_NONE;
		FName PolygonGroupImportedMaterialSlotName = NAME_None;
		int32 MaterialIndex = SourceRawMesh.FaceMaterialIndices[TriangleIndex];
		if (MaterialIndexToPolygonGroup.Contains(MaterialIndex))
		{
			PolygonGroupID = MaterialIndexToPolygonGroup[MaterialIndex];
		}
		else if (MaterialMap.Num() > 0 && MaterialMap.Contains(MaterialIndex))
		{
			PolygonGroupImportedMaterialSlotName = MaterialMap[MaterialIndex];
			for (const FPolygonGroupID SearchPolygonGroupID : DestinationMeshDescription.PolygonGroups().GetElementIDs())
			{
				if (PolygonGroupImportedMaterialSlotNames[SearchPolygonGroupID] == PolygonGroupImportedMaterialSlotName)
				{
					PolygonGroupID = SearchPolygonGroupID;
					break;
				}
			}
		}

		if (PolygonGroupID == INDEX_NONE)
		{
			PolygonGroupID = DestinationMeshDescription.CreatePolygonGroup();
			PolygonGroupImportedMaterialSlotNames[PolygonGroupID] = (PolygonGroupImportedMaterialSlotName == NAME_None) ? FName(*FString::Printf(TEXT("MaterialSlot_%d"), MaterialIndex)) : PolygonGroupImportedMaterialSlotName;
			PolygonGroups.Add(PolygonGroupID);
			MaterialIndexToPolygonGroup.Add(MaterialIndex, PolygonGroupID);
		}
		FVertexInstanceID TriangleVertexInstanceIDs[3];
		for (int32 Corner = 0; Corner < 3; ++Corner)
		{
			int32 VerticeIndex = VerticeIndexBase + Corner;
			FVertexID VertexID = RemapVertexPosition[SourceRawMesh.WedgeIndices[VerticeIndex]];
			FVertexInstanceID VertexInstanceID = DestinationMeshDescription.CreateVertexInstance(VertexID);
			TriangleVertexInstanceIDs[Corner] = VertexInstanceID;
			VertexInstanceColors[VertexInstanceID] = bHasColors ? FVector4f(FLinearColor::FromSRGBColor(SourceRawMesh.WedgeColors[VerticeIndex])) : FVector4f(FLinearColor::White);
			VertexInstanceNormals[VertexInstanceID] = bHasNormals ? SourceRawMesh.WedgeTangentZ[VerticeIndex] : FVector3f(ForceInitToZero);

			if (bHasTangents)
			{
				VertexInstanceTangents[VertexInstanceID] = SourceRawMesh.WedgeTangentX[VerticeIndex];
				VertexInstanceBinormalSigns[VertexInstanceID] = FMatrix44f(SourceRawMesh.WedgeTangentX[VerticeIndex].GetSafeNormal(),
					SourceRawMesh.WedgeTangentY[VerticeIndex].GetSafeNormal(),
					SourceRawMesh.WedgeTangentZ[VerticeIndex].GetSafeNormal(),
					FVector3f::ZeroVector).Determinant() < 0 ? -1.0f : +1.0f;
			}
			else
			{
				VertexInstanceTangents[VertexInstanceID] = FVector3f(ForceInitToZero);
				VertexInstanceBinormalSigns[VertexInstanceID] = 0.0f;
			}

			for (int32 TextureCoordinnateIndex = 0; TextureCoordinnateIndex < NumTexCoords; ++TextureCoordinnateIndex)
			{
				int32 TextureCoordIndex = TextureCoordinnateRemapIndex[TextureCoordinnateIndex];
				if (TextureCoordIndex == INDEX_NONE)
				{
					continue;
				}
				VertexInstanceUVs.Set(VertexInstanceID, TextureCoordIndex, SourceRawMesh.WedgeTexCoords[TextureCoordinnateIndex][VerticeIndex]);
			}
		}

		DestinationMeshDescription.CreateTriangle(PolygonGroupID, TriangleVertexInstanceIDs);
	}

	ConvertSmoothGroupToHardEdges(SourceRawMesh.FaceSmoothingMasks, DestinationMeshDescription);

	//Create the missing normals and tangents, should we use Mikkt space for tangent???
	if (!bSkipNormalsAndTangents && (!bHasNormals || !bHasTangents))
	{
		//DestinationMeshDescription.ComputePolygonTangentsAndNormals(0.0f);
		ComputeTriangleTangentsAndNormals(DestinationMeshDescription, 0.0f, DebugName);

		//Create the missing normals and recompute the tangents with MikkTSpace.
		EComputeNTBsFlags ComputeNTBsOptions = EComputeNTBsFlags::Tangents | EComputeNTBsFlags::UseMikkTSpace | EComputeNTBsFlags::BlendOverlappingNormals;
		ComputeTangentsAndNormals(DestinationMeshDescription, ComputeNTBsOptions);
	}

	DestinationMeshDescription.BuildIndexers();
}

void FStaticMeshOperations::AppendMeshDescriptions(const TArray<const FMeshDescription*>& SourceMeshes, FMeshDescription& TargetMesh, const FAppendSettings& AppendSettings)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshOperations::AppendMeshDescriptions);

	FStaticMeshAttributes TargetAttributes(TargetMesh);
	TVertexAttributesRef<FVector3f> TargetVertexPositions = TargetAttributes.GetVertexPositions();
	TEdgeAttributesRef<bool> TargetEdgeHardnesses = TargetAttributes.GetEdgeHardnesses();
	TPolygonGroupAttributesRef<FName> TargetImportedMaterialSlotNames = TargetAttributes.GetPolygonGroupMaterialSlotNames();
	TVertexInstanceAttributesRef<FVector3f> TargetVertexInstanceNormals = TargetAttributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> TargetVertexInstanceTangents = TargetAttributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> TargetVertexInstanceBinormalSigns = TargetAttributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesRef<FVector4f> TargetVertexInstanceColors = TargetAttributes.GetVertexInstanceColors();
	TVertexInstanceAttributesRef<FVector2f> TargetVertexInstanceUVs = TargetAttributes.GetVertexInstanceUVs();

	TPolygonAttributesRef<FName> TargetPolygonObjectNames;
	if (AppendSettings.bMergeObjectName)
	{
		// If any of the source meshes have an object name attribute, but the target mesh doesn't, we register it here so that we can
		// actually honor the bMergeObjectNames flag.
		if (!TargetAttributes.GetPolygonObjectNames().IsValid())
		{
			for (const FMeshDescription* SourceMesh: SourceMeshes)
			{
				FStaticMeshConstAttributes SourceAttributes(*SourceMesh);
				if (SourceAttributes.GetPolygonObjectNames().IsValid())
				{
					TargetAttributes.RegisterPolygonObjectNameAttribute();
					break;
				}
			}
		}
		TargetPolygonObjectNames = TargetAttributes.GetPolygonObjectNames();
	}

	TargetMesh.SuspendVertexInstanceIndexing();
	TargetMesh.SuspendEdgeIndexing();
	TargetMesh.SuspendPolygonIndexing();
	TargetMesh.SuspendPolygonGroupIndexing();
	TargetMesh.SuspendUVIndexing();

	int32 NumVertices = 0;
	int32 NumVertexInstances = 0;
	int32 NumEdges = 0;
	int32 NumTriangles = 0;

	int32 MaxNumVertexInstanceUVChannels = TargetVertexInstanceUVs.GetNumChannels();
	int32 MaxNumUVChannels = TargetMesh.GetNumUVElementChannels();
	int32 MaxNumPolygonGroups = 0;
	int32 MaxNumMeshVertices = 0;
	int32 MaxNumEdges = 0;
	int32 MaxNumVertexInstances = 0;

	for (const FMeshDescription* SourceMeshPtr : SourceMeshes)
	{
		const FMeshDescription& SourceMesh = *SourceMeshPtr;

		NumVertices += SourceMesh.Vertices().Num();
		NumVertexInstances += SourceMesh.VertexInstances().Num();
		NumEdges += SourceMesh.Edges().Num();
		NumTriangles += SourceMesh.Triangles().Num();

		FStaticMeshConstAttributes SourceAttributes(SourceMesh);
		TVertexInstanceAttributesConstRef<FVector2f> SourceVertexInstanceUVs = SourceAttributes.GetVertexInstanceUVs();
		for (int32 ChannelIdx = MaxNumVertexInstanceUVChannels; ChannelIdx < SourceVertexInstanceUVs.GetNumChannels(); ++ChannelIdx)
		{
			if (AppendSettings.bMergeUVChannels[ChannelIdx])
			{
				MaxNumVertexInstanceUVChannels = ChannelIdx + 1;
			}
		}
		for (int32 ChannelIdx = MaxNumUVChannels; ChannelIdx < SourceMesh.GetNumUVElementChannels(); ++ChannelIdx)
		{
			if (AppendSettings.bMergeUVChannels[ChannelIdx])
			{
				MaxNumUVChannels = ChannelIdx + 1;
			}
		}

		MaxNumPolygonGroups = FMath::Max(MaxNumPolygonGroups, SourceMesh.PolygonGroups().Num());
		MaxNumMeshVertices = FMath::Max(MaxNumMeshVertices, SourceMesh.Vertices().Num());
		MaxNumEdges = FMath::Max(MaxNumEdges, SourceMesh.Edges().Num());
		MaxNumVertexInstances = FMath::Max(MaxNumVertexInstances, SourceMesh.VertexInstances().Num());
	}

	//Copy into the target mesh
	TargetMesh.ReserveNewVertices(NumVertices);
	TargetMesh.ReserveNewVertexInstances(NumVertexInstances);
	TargetMesh.ReserveNewEdges(NumEdges);
	TargetMesh.ReserveNewTriangles(NumTriangles);

	if (MaxNumVertexInstanceUVChannels > TargetVertexInstanceUVs.GetNumChannels())
	{
		TargetVertexInstanceUVs.SetNumChannels(MaxNumVertexInstanceUVChannels);
	}
	if (MaxNumUVChannels > TargetMesh.GetNumUVElementChannels())
	{
		TargetMesh.SetNumUVChannels(MaxNumUVChannels);
	}

	PolygonGroupMap RemapPolygonGroup;

	TMap<FVertexID, FVertexID> SourceToTargetVertexID;
	SourceToTargetVertexID.Reserve(MaxNumMeshVertices);
	TMap<FVertexInstanceID, FVertexInstanceID> SourceToTargetVertexInstanceID;
	SourceToTargetVertexInstanceID.Reserve(MaxNumVertexInstances);

	for (const FMeshDescription* SourceMeshPtr : SourceMeshes)
	{
		const FMeshDescription& SourceMesh = *SourceMeshPtr;

		RemapPolygonGroup.Empty(MaxNumPolygonGroups);

		FStaticMeshConstAttributes SourceAttributes(SourceMesh);
		TVertexAttributesConstRef<FVector3f> SourceVertexPositions = SourceAttributes.GetVertexPositions();
		TEdgeAttributesConstRef<bool> SourceEdgeHardnesses = SourceAttributes.GetEdgeHardnesses();
		TPolygonGroupAttributesConstRef<FName> SourceImportedMaterialSlotNames = SourceAttributes.GetPolygonGroupMaterialSlotNames();
		TVertexInstanceAttributesConstRef<FVector3f> SourceVertexInstanceNormals = SourceAttributes.GetVertexInstanceNormals();
		TVertexInstanceAttributesConstRef<FVector3f> SourceVertexInstanceTangents = SourceAttributes.GetVertexInstanceTangents();
		TVertexInstanceAttributesConstRef<float> SourceVertexInstanceBinormalSigns = SourceAttributes.GetVertexInstanceBinormalSigns();
		TVertexInstanceAttributesConstRef<FVector4f> SourceVertexInstanceColors = SourceAttributes.GetVertexInstanceColors();
		TVertexInstanceAttributesConstRef<FVector2f> SourceVertexInstanceUVs = SourceAttributes.GetVertexInstanceUVs();

		TPolygonAttributesConstRef<FName> SourcePolygonObjectNames;
		if (TargetPolygonObjectNames.IsValid())
		{
			SourcePolygonObjectNames = SourceAttributes.GetPolygonObjectNames();
		}
		
		// Fill the UV arrays
		const int32 NumUVChannel = FMath::Min(TargetMesh.GetNumUVElementChannels(), SourceMesh.GetNumUVElementChannels());
		for (int32 UVLayerIndex = 0; UVLayerIndex < NumUVChannel; UVLayerIndex++)
		{
			TUVAttributesConstRef<FVector2f> SourceUVCoordinates = SourceMesh.UVAttributes(UVLayerIndex).GetAttributesRef<FVector2f>(MeshAttribute::UV::UVCoordinate);
			TUVAttributesRef<FVector2f> TargetUVCoordinates = TargetMesh.UVAttributes(UVLayerIndex).GetAttributesRef<FVector2f>(MeshAttribute::UV::UVCoordinate);
			int32 UVCount = SourceUVCoordinates.GetNumElements();
			TargetMesh.ReserveNewUVs(UVCount, UVLayerIndex);
			for (FUVID SourceUVID : SourceMesh.UVs(UVLayerIndex).GetElementIDs())
			{
				FUVID TargetUVID = TargetMesh.CreateUV(UVLayerIndex);
				TargetUVCoordinates[TargetUVID] = SourceUVCoordinates[SourceUVID];
			}
		}

		//PolygonGroups
		if (AppendSettings.PolygonGroupsDelegate.IsBound())
		{
			AppendSettings.PolygonGroupsDelegate.Execute(SourceMesh, TargetMesh, RemapPolygonGroup);
		}
		else
		{			
			for (FPolygonGroupID SourcePolygonGroupID : SourceMesh.PolygonGroups().GetElementIDs())
			{
				FPolygonGroupID TargetMatchingID = INDEX_NONE;
				for (FPolygonGroupID TargetPolygonGroupID : TargetMesh.PolygonGroups().GetElementIDs())
				{
					if (SourceImportedMaterialSlotNames[SourcePolygonGroupID] == TargetImportedMaterialSlotNames[TargetPolygonGroupID])
					{
						TargetMatchingID = TargetPolygonGroupID;
						break;
					}
				}
				if (TargetMatchingID == INDEX_NONE)
				{
					TargetMatchingID = TargetMesh.CreatePolygonGroup();
					TargetImportedMaterialSlotNames[TargetMatchingID] = SourceImportedMaterialSlotNames[SourcePolygonGroupID];
				}
				RemapPolygonGroup.Add(SourcePolygonGroupID, TargetMatchingID);
			}
		}

		FPolygonGroupID SinglePolygonGroup = TargetMesh.PolygonGroups().Num() == 1 ? TargetMesh.PolygonGroups().GetFirstValidID() : INDEX_NONE;

		//Vertices
		for (FVertexID SourceVertexID : SourceMesh.Vertices().GetElementIDs())
		{
			FVertexID TargetVertexID = TargetMesh.CreateVertex();
			TargetVertexPositions[TargetVertexID] = (SourceVertexPositions[SourceVertexID] - FVector3f(AppendSettings.MergedAssetPivot));	//LWC_TODO: Precision loss

			SourceToTargetVertexID.Add(SourceVertexID, TargetVertexID);
		}

		// Transform vertices properties
		if (AppendSettings.MeshTransform)
		{
			const FTransform& Transform = AppendSettings.MeshTransform.GetValue();
			for (const TPair<FVertexID, FVertexID>& VertexIDPair : SourceToTargetVertexID)
			{
				FVector3f& Position = TargetVertexPositions[VertexIDPair.Value];
				Position = FVector3f(Transform.TransformPosition(FVector(Position)));	//LWC_TODO: Precision loss
			}
		}

		//Edges
		for (const FEdgeID SourceEdgeID : SourceMesh.Edges().GetElementIDs())
		{
			const FVertexID EdgeVertex0 = SourceMesh.GetEdgeVertex(SourceEdgeID, 0);
			const FVertexID EdgeVertex1 = SourceMesh.GetEdgeVertex(SourceEdgeID, 1);
			FEdgeID TargetEdgeID = TargetMesh.CreateEdge(SourceToTargetVertexID[EdgeVertex0], SourceToTargetVertexID[EdgeVertex1]);
			TargetEdgeHardnesses[TargetEdgeID] = SourceEdgeHardnesses[SourceEdgeID];
		}

		//VertexInstances
		for (const FVertexInstanceID SourceVertexInstanceID : SourceMesh.VertexInstances().GetElementIDs())
		{
			const FVertexID SourceVertexID = SourceMesh.GetVertexInstanceVertex(SourceVertexInstanceID);
			FVertexInstanceID TargetVertexInstanceID = TargetMesh.CreateVertexInstance(SourceToTargetVertexID[SourceVertexID]);
	
			TargetVertexInstanceNormals[TargetVertexInstanceID] = SourceVertexInstanceNormals[SourceVertexInstanceID];
			TargetVertexInstanceTangents[TargetVertexInstanceID] = SourceVertexInstanceTangents[SourceVertexInstanceID];
			TargetVertexInstanceBinormalSigns[TargetVertexInstanceID] = SourceVertexInstanceBinormalSigns[SourceVertexInstanceID];

			if (AppendSettings.bMergeVertexColor)
			{
				TargetVertexInstanceColors[TargetVertexInstanceID] = SourceVertexInstanceColors[SourceVertexInstanceID];
			}

			for (int32 UVChannelIndex = 0; UVChannelIndex < MaxNumVertexInstanceUVChannels && UVChannelIndex < SourceVertexInstanceUVs.GetNumChannels(); ++UVChannelIndex)
			{
				TargetVertexInstanceUVs.Set(TargetVertexInstanceID, UVChannelIndex, SourceVertexInstanceUVs.Get(SourceVertexInstanceID, UVChannelIndex));
			}

			SourceToTargetVertexInstanceID.Add(SourceVertexInstanceID, TargetVertexInstanceID);
		}

		bool bReverseCulling = false;
		// Transform vertex instances properties
		if (AppendSettings.MeshTransform)
		{
			const FTransform& Transform = AppendSettings.MeshTransform.GetValue();
			FMatrix TransformInverseTransposeMatrix = Transform.ToMatrixWithScale().Inverse().GetTransposed();
			TransformInverseTransposeMatrix.RemoveScaling();

			bReverseCulling = Transform.GetDeterminant() < 0;
			float BinormalSignsFactor = bReverseCulling ? -1.f : 1.f;
			for (const TPair<FVertexInstanceID, FVertexInstanceID>& VertexInstanceIDPair : SourceToTargetVertexInstanceID)
			{
				FVertexInstanceID InstanceID = VertexInstanceIDPair.Value;

				FVector3f& Normal = TargetVertexInstanceNormals[InstanceID];
				Normal = static_cast<FVector3f>(FVector(TransformInverseTransposeMatrix.TransformVector(static_cast<FVector>(Normal)).GetSafeNormal()));

				FVector3f& Tangent = TargetVertexInstanceTangents[InstanceID];
				Tangent = static_cast<FVector3f>(FVector(TransformInverseTransposeMatrix.TransformVector(static_cast<FVector>(Tangent)).GetSafeNormal()));

				TargetVertexInstanceBinormalSigns[InstanceID] *= BinormalSignsFactor;
			}
		}

		// Polygons
		TArray<FVertexInstanceID> PolygonVertexInstanceIDs;
		PolygonVertexInstanceIDs.Reserve(8);
		
		for (const FPolygonID SourcePolygonID: SourceMesh.Polygons().GetElementIDs())
		{
			SourceMesh.GetPolygonVertexInstances(SourcePolygonID, PolygonVertexInstanceIDs);

			//Find the polygonGroupID
			FPolygonGroupID TargetPolygonGroupID = SinglePolygonGroup != INDEX_NONE ? SinglePolygonGroup : RemapPolygonGroup[SourceMesh.GetPolygonPolygonGroup(SourcePolygonID)];

			TArray<FVertexInstanceID, TInlineAllocator<4>> VertexInstanceIDs;
			VertexInstanceIDs.Reserve(4);
			if (bReverseCulling)
			{
				for (int32 ReverseVertexInstanceIdIndex = PolygonVertexInstanceIDs.Num()-1; ReverseVertexInstanceIdIndex >= 0; ReverseVertexInstanceIdIndex--)
				{
					VertexInstanceIDs.Add(SourceToTargetVertexInstanceID[PolygonVertexInstanceIDs[ReverseVertexInstanceIdIndex]]);
				}
			}
			else
			{
				for (const FVertexInstanceID& VertexInstanceID : PolygonVertexInstanceIDs)
				{
					VertexInstanceIDs.Add(SourceToTargetVertexInstanceID[VertexInstanceID]);
				}
			}

			FPolygonID TargetPolygonID = TargetMesh.CreatePolygon(TargetPolygonGroupID, VertexInstanceIDs);

			// Copy the source object name, if requested. If the attribute doesn't exist on the source, just leave the name at default.
			// SourcePolygonObjectNames can only be valid if TargetPolygonObjectNames, so we only check that here.
			if (SourcePolygonObjectNames.IsValid())
			{
				TargetPolygonObjectNames.Set(TargetPolygonID, SourcePolygonObjectNames.Get(SourcePolygonID));
			}
		}
	}

	TargetMesh.ResumeVertexInstanceIndexing();
	TargetMesh.ResumeEdgeIndexing();
	TargetMesh.ResumePolygonIndexing();
	TargetMesh.ResumePolygonGroupIndexing();
	TargetMesh.ResumeUVIndexing();
}

void FStaticMeshOperations::AppendMeshDescription(const FMeshDescription& SourceMesh, FMeshDescription& TargetMesh, const FAppendSettings& AppendSettings)
{
	AppendMeshDescriptions({ &SourceMesh }, TargetMesh, AppendSettings);
}

//////////////////////////////////////////////////////////////////////////
// Normals tangents and Bi-normals
void FStaticMeshOperations::HasInvalidVertexInstanceNormalsOrTangents(const FMeshDescription& MeshDescription, bool& bHasInvalidNormals, bool& bHasInvalidTangents)
{
	bHasInvalidNormals = false;
	bHasInvalidTangents = false;

	FStaticMeshConstAttributes Attributes(MeshDescription);
	TArrayView<const FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals().GetRawArray();
	TArrayView<const FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents().GetRawArray();

	for (const FVertexInstanceID VertexInstanceID : MeshDescription.VertexInstances().GetElementIDs())
	{
		bHasInvalidNormals |= (VertexInstanceNormals[VertexInstanceID].IsNearlyZero() || VertexInstanceNormals[VertexInstanceID].ContainsNaN());
		bHasInvalidTangents |= (VertexInstanceTangents[VertexInstanceID].IsNearlyZero() || VertexInstanceTangents[VertexInstanceID].ContainsNaN());
		if (bHasInvalidNormals && bHasInvalidTangents)
		{
			break;
		}
	}
}

void ClearNormalsAndTangentsData(FMeshDescription& MeshDescription, bool bClearNormals, bool bClearTangents)
{
	if (!bClearNormals && !bClearTangents)
	{
		return;
	}

	FStaticMeshAttributes Attributes(MeshDescription);
	TArrayView<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals().GetRawArray();
	TArrayView<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents().GetRawArray();
	TArrayView<float> VertexInstanceBinormals = Attributes.GetVertexInstanceBinormalSigns().GetRawArray();

	//Zero out all value that need to be recompute
	for (const FVertexInstanceID VertexInstanceID : MeshDescription.VertexInstances().GetElementIDs())
	{
		if (bClearNormals)
		{
			VertexInstanceNormals[VertexInstanceID] = FVector3f::ZeroVector;
		}
		if (bClearTangents)
		{
			// Dump the tangents
			VertexInstanceBinormals[VertexInstanceID] = 0.0f;
			VertexInstanceTangents[VertexInstanceID] = FVector3f::ZeroVector;
		}
	}
}

struct FNTBGroupKeyFuncs : public TDefaultMapKeyFuncs<FVector2f, FVector3f, false>
{
	//We need to sanitize the key here to make sure -0.0f fall on the same hash then 0.0f
	static FORCEINLINE_DEBUGGABLE uint32 GetKeyHash(KeyInitType Key)
	{
		FVector2f TmpKey;
		TmpKey.X = FMath::IsNearlyZero(Key.X) ? 0.0f : Key.X;
		TmpKey.Y = FMath::IsNearlyZero(Key.Y) ? 0.0f : Key.Y;
		return FCrc::MemCrc32(&TmpKey, sizeof(TmpKey));
	}
};

void FStaticMeshOperations::RecomputeNormalsAndTangentsIfNeeded(FMeshDescription& MeshDescription, EComputeNTBsFlags ComputeNTBsOptions)
{
	if (!EnumHasAllFlags(ComputeNTBsOptions, EComputeNTBsFlags::Normals | EComputeNTBsFlags::Tangents))
	{
		bool bRecomputeNormals = false;
		bool bRecomputeTangents = false;
		
		HasInvalidVertexInstanceNormalsOrTangents(MeshDescription, bRecomputeNormals, bRecomputeTangents);
		
		ComputeNTBsOptions |= (bRecomputeNormals ? EComputeNTBsFlags::Normals : EComputeNTBsFlags::None);
		ComputeNTBsOptions |= (bRecomputeTangents ? EComputeNTBsFlags::Tangents : EComputeNTBsFlags::None);
	}

	if (EnumHasAnyFlags(ComputeNTBsOptions, EComputeNTBsFlags::Normals | EComputeNTBsFlags::Tangents))
	{
		ComputeTangentsAndNormals(MeshDescription, ComputeNTBsOptions);
	}
}

void FStaticMeshOperations::ComputeTangentsAndNormals(FMeshDescription& MeshDescription, EComputeNTBsFlags ComputeNTBsOptions)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshOperations::ComputeTangentsAndNormals);

	//For each vertex compute the normals for every connected edges that are smooth between hard edges
	//         H   A    B
	//          \  ||  /
	//       G  -- ** -- C
	//          // |  \
	//         F   E    D
	//
	// The double ** are the vertex, the double line are hard edges, the single line are soft edge.
	// A and F are hard, all other edges are soft. The goal is to compute two average normals one from
	// A to F and a second from F to A. Then we can set the vertex instance normals accordingly.
	// First normal(A to F) = Normalize(A+B+C+D+E+F)
	// Second normal(F to A) = Normalize(F+G+H+A)
	// We found the connected edge using the triangle that share edges

	struct FTriangleCornerData
	{
		FVertexInstanceID VertexInstanceID;
		float CornerAngle;
	};

	struct FTriangleData
	{
	public:
		//The area of the triangle
		float Area;

		//Set the corner angle data for a FVertexInstanceID
		void SetCornerAngleData(FVertexInstanceID VertexInstanceID, float CornerAngle, int32 CornerIndex)
		{
			CornerAngleDatas[CornerIndex].VertexInstanceID = VertexInstanceID;
			CornerAngleDatas[CornerIndex].CornerAngle = CornerAngle;
		}

		//Get the angle for the FVertexInstanceID
		float GetCornerAngle(FVertexInstanceID VertexInstanceID)
		{
			for (int32 CornerIndex = 0; CornerIndex < 3; ++CornerIndex)
			{
				if (CornerAngleDatas[CornerIndex].VertexInstanceID == VertexInstanceID)
				{
					return CornerAngleDatas[CornerIndex].CornerAngle;
				}
			}

			//We should always found a valid VertexInstanceID
			check(false);
			return 0.0f;
		}
	private:
		//The data for each corner
		FTriangleCornerData CornerAngleDatas[3];
	};

#if 0
	//Make sure the meshdescription is triangulate
	if (MeshDescription.Triangles().Num() < MeshDescription.Polygons().Num())
	{
		//Triangulate the mesh, we compute the normals on triangle not on polygon.
		MeshDescription.TriangulateMesh();
	}
#endif

	if (MeshDescription.Triangles().Num() == 0)
	{
		return;
	}

	const bool bForceComputeNormals = EnumHasAllFlags(ComputeNTBsOptions, EComputeNTBsFlags::Normals);
	const bool bForceComputeTangent = EnumHasAnyFlags(ComputeNTBsOptions, EComputeNTBsFlags::Normals | EComputeNTBsFlags::Tangents);
	const bool bComputeTangentWithMikkTSpace = bForceComputeTangent && EnumHasAllFlags(ComputeNTBsOptions, EComputeNTBsFlags::UseMikkTSpace);
	const bool bComputeWeightedNormals = EnumHasAllFlags(ComputeNTBsOptions, EComputeNTBsFlags::WeightedNTBs);

	//Clear any data we want to force-recompute since the following code actually look for any invalid data and recompute it.
	ClearNormalsAndTangentsData(MeshDescription, bForceComputeNormals, bForceComputeTangent);

	// Going to iterate over all triangles, so mandate that the triangle elements are compact, i.e. there are no holes
	const int32 NumTriangles = MeshDescription.Triangles().Num();
	check(MeshDescription.Triangles().GetArraySize() == NumTriangles);

	// Compute the weight (area and angle) for each triangles
	TArray<FTriangleData> TriangleDatas;
	TriangleDatas.SetNum(NumTriangles);
	if (bComputeWeightedNormals)
	{
		FStaticMeshAttributes Attributes(MeshDescription);
		TArrayView<const FVector3f> VertexPositions = Attributes.GetVertexPositions().GetRawArray();
		TArrayView<const FVertexID> TriVertexIDs = Attributes.GetTriangleVertexIndices().GetRawArray();
		TArrayView<const FVertexInstanceID> TriVertexInstanceIDs = Attributes.GetTriangleVertexInstanceIndices().GetRawArray();

		TriangleDatas.Reserve(NumTriangles);

		for (int32 Index = 0, TriIndex = 0; Index < NumTriangles; Index++, TriIndex += 3)
		{
			const FVector3f PointA(VertexPositions[TriVertexIDs[TriIndex + 0]]);
			const FVector3f PointB(VertexPositions[TriVertexIDs[TriIndex + 1]]);
			const FVector3f PointC(VertexPositions[TriVertexIDs[TriIndex + 2]]);
			FTriangleData& TriangleData = TriangleDatas[Index];
			TriangleData.Area = TriangleUtilities::ComputeTriangleArea(PointA, PointB, PointC);
			TriangleData.SetCornerAngleData(TriVertexInstanceIDs[TriIndex + 0], TriangleUtilities::ComputeTriangleCornerAngle(PointA, PointB, PointC), 0);
			TriangleData.SetCornerAngleData(TriVertexInstanceIDs[TriIndex + 1], TriangleUtilities::ComputeTriangleCornerAngle(PointB, PointC, PointA), 1);
			TriangleData.SetCornerAngleData(TriVertexInstanceIDs[TriIndex + 2], TriangleUtilities::ComputeTriangleCornerAngle(PointC, PointA, PointB), 2);
		}
	}

	// Ensure certain indexers are built in anticipation
	MeshDescription.BuildVertexIndexers();
	MeshDescription.BuildEdgeIndexers();

	// Going to iterate over all vertices, so mandate that the vertex elements are compact, i.e. there are no holes
	const int32 NumVertices = MeshDescription.Vertices().Num();
	check(MeshDescription.Vertices().GetArraySize() == NumVertices);

	// Split work in batch to reduce call and allocation overhead
	const int32 BatchSize = 128 * 1024;
	const int32 BatchCount = (NumVertices + BatchSize - 1) / BatchSize;

	//Iterate all vertex to compute normals for all vertex instance
	ParallelFor( TEXT("ComputeTangentsAndNormals.PF"), BatchCount,1,
		[NumVertices, BatchSize, bComputeTangentWithMikkTSpace, bComputeWeightedNormals, &MeshDescription, &TriangleDatas](int32 BatchIndex)
		{
			FStaticMeshAttributes Attributes(MeshDescription);

			TArrayView<const FVector2f> VertexUVs;
			TArrayView<const FVector3f> TriangleNormals = Attributes.GetTriangleNormals().GetRawArray();
			TArrayView<const FVector3f> TriangleTangents = Attributes.GetTriangleTangents().GetRawArray();
			TArrayView<const FVector3f> TriangleBinormals = Attributes.GetTriangleBinormals().GetRawArray();
			TArrayView<const bool> EdgeHardnesses = Attributes.GetEdgeHardnesses().GetRawArray();

			TArrayView<FVector3f> VertexNormals = Attributes.GetVertexInstanceNormals().GetRawArray();
			TArrayView<FVector3f> VertexTangents = Attributes.GetVertexInstanceTangents().GetRawArray();
			TArrayView<float> VertexBinormalSigns = Attributes.GetVertexInstanceBinormalSigns().GetRawArray();

			// If the mesh has no UVs, average all tangents/bi-normals for a given vertex, rather than try to maintain
			// the UV flow.
			if (Attributes.GetVertexInstanceUVs().IsValid() && Attributes.GetVertexInstanceUVs().GetNumChannels() > 0)
			{
				// Use UV0 as the base. Same as with ComputeTriangleTangentsAndNormals 
				VertexUVs = Attributes.GetVertexInstanceUVs().GetRawArray(0);
			}
			
			check(TriangleNormals.Num() > 0);
			check(TriangleTangents.Num() > 0);
			check(TriangleBinormals.Num() > 0);

			//Reuse containers between iterations to reduce allocations
			TMap<FVector2f, FVector3f, FDefaultSetAllocator, FNTBGroupKeyFuncs> GroupTangent;
			TMap<FVector2f, FVector3f, FDefaultSetAllocator, FNTBGroupKeyFuncs> GroupBiNormal;
			TMap<FTriangleID, FVertexInfo> VertexInfoMap;
			TArray<TArray<FTriangleID, TInlineAllocator<8>>> Groups;
			TSet<FTriangleID> ConsumedTriangle;
			TArray<FTriangleID> PolygonQueue;
			TArray<FVertexInstanceID> VertexInstanceInGroup;

			VertexInfoMap.Reserve(20);

			int32 StartIndex = BatchIndex * BatchSize;
			int32 LastIndex = FMath::Min(StartIndex + BatchSize, NumVertices);
			for (int32 Index = StartIndex; Index < LastIndex; ++Index)
			{
				VertexInfoMap.Reset();

				const FVertexID VertexID(Index);

				bool bPointHasAllTangents = true;
				// Fill the VertexInfoMap
				for (const FEdgeID& EdgeID : MeshDescription.GetVertexConnectedEdgeIDs(VertexID))
				{
					for (const FTriangleID& TriangleID : MeshDescription.GetEdgeConnectedTriangleIDs(EdgeID))
					{
						FVertexInfo& VertexInfo = VertexInfoMap.FindOrAdd(TriangleID);
						int32 EdgeIndex = VertexInfo.EdgeIDs.AddUnique(EdgeID);
						if (VertexInfo.TriangleID == INDEX_NONE)
						{
							VertexInfo.TriangleID = TriangleID;
							for (const FVertexInstanceID& VertexInstanceID : MeshDescription.GetTriangleVertexInstances(TriangleID))
							{
								if (MeshDescription.GetVertexInstanceVertex(VertexInstanceID) == VertexID)
								{
									VertexInfo.VertexInstanceID = VertexInstanceID;
									if (!VertexUVs.IsEmpty())
									{
										VertexInfo.UVs = VertexUVs[VertexInstanceID];	// UV0
									}
									bPointHasAllTangents &= !VertexNormals[VertexInstanceID].IsNearlyZero() && !VertexTangents[VertexInstanceID].IsNearlyZero();
									if (bPointHasAllTangents)
									{
										FVector3f TangentX = VertexTangents[VertexInstanceID].GetSafeNormal();
										FVector3f TangentZ = VertexNormals[VertexInstanceID].GetSafeNormal();
										FVector3f TangentY = (FVector3f::CrossProduct(TangentZ, TangentX).GetSafeNormal() * VertexBinormalSigns[VertexInstanceID]).GetSafeNormal();
										if (TangentX.ContainsNaN() || TangentX.IsNearlyZero(SMALL_NUMBER) ||
											TangentY.ContainsNaN() || TangentY.IsNearlyZero(SMALL_NUMBER) ||
											TangentZ.ContainsNaN() || TangentZ.IsNearlyZero(SMALL_NUMBER))
										{
											bPointHasAllTangents = false;
										}
									}
									break;
								}
							}
						}
					}
				}

				if (bPointHasAllTangents)
				{
					continue;
				}

				//Build all group by recursively traverse all polygon connected to the vertex
				Groups.Reset();
				ConsumedTriangle.Reset();
				TSet<FEdgeID> ConsumedEdges;
				for (auto Kvp : VertexInfoMap)
				{
					if (ConsumedTriangle.Contains(Kvp.Key))
					{
						continue;
					}

					int32 CurrentGroupIndex = Groups.AddZeroed();
					TArray<FTriangleID, TInlineAllocator<8>>& CurrentGroup = Groups[CurrentGroupIndex];
					PolygonQueue.Reset();
					PolygonQueue.Add(Kvp.Key); //Use a queue to avoid recursive function
					while (PolygonQueue.Num() > 0)
					{
						FTriangleID CurrentPolygonID = PolygonQueue.Pop(EAllowShrinking::No);
						FVertexInfo& CurrentVertexInfo = VertexInfoMap.FindOrAdd(CurrentPolygonID);
						CurrentGroup.AddUnique(CurrentVertexInfo.TriangleID);
						ConsumedTriangle.Add(CurrentVertexInfo.TriangleID);
						for (const FEdgeID& EdgeID : CurrentVertexInfo.EdgeIDs)
						{
							if (EdgeHardnesses[EdgeID] || ConsumedEdges.Contains(EdgeID))
							{
								//End of the group or non manifold edge
								continue;
							}

							ConsumedEdges.Add(EdgeID);

							for (const FTriangleID& TriangleID : MeshDescription.GetEdgeConnectedTriangleIDs(EdgeID))
							{
								if (TriangleID == CurrentVertexInfo.TriangleID)
								{
									continue;
								}
								//Add this polygon to the group
								FVertexInfo& OtherVertexInfo = VertexInfoMap.FindOrAdd(TriangleID);
								//Do not repeat polygons
								if (!ConsumedTriangle.Contains(OtherVertexInfo.TriangleID))
								{
									PolygonQueue.Add(TriangleID);
								}
							}
						}
					}
				}

				for (const TArray<FTriangleID, TInlineAllocator<8>>& Group : Groups)
				{
					//Compute tangents data
					GroupTangent.Reset();
					GroupBiNormal.Reset();
					VertexInstanceInGroup.Reset();

					FVector3f GroupNormal(FVector3f::ZeroVector);
					
					for (const FTriangleID& TriangleID : Group)
					{
						FVertexInfo& CurrentVertexInfo = VertexInfoMap.FindOrAdd(TriangleID);
						float CornerWeight = 1.0f;

						if (bComputeWeightedNormals)
						{
							FTriangleData& TriangleData = TriangleDatas[TriangleID];
							CornerWeight = TriangleData.Area * TriangleData.GetCornerAngle(CurrentVertexInfo.VertexInstanceID);
						}

						const FVector3f TriNormal = CornerWeight * TriangleNormals[TriangleID];
						const FVector3f TriTangent = CornerWeight * TriangleTangents[TriangleID];
						const FVector3f TriBinormal = CornerWeight * TriangleBinormals[TriangleID];

						VertexInstanceInGroup.Add(VertexInfoMap[TriangleID].VertexInstanceID);
						if (!TriNormal.IsNearlyZero(SMALL_NUMBER) && !TriNormal.ContainsNaN())
						{
							GroupNormal += TriNormal;
						}
						if (!bComputeTangentWithMikkTSpace)
						{
							const FVector2f& UVs = VertexInfoMap[TriangleID].UVs;
							bool CreateGroup = (!GroupTangent.Contains(UVs));
							FVector3f& GroupTangentValue = GroupTangent.FindOrAdd(UVs);
							FVector3f& GroupBiNormalValue = GroupBiNormal.FindOrAdd(UVs);
							if (CreateGroup)
							{
								GroupTangentValue = FVector3f(0.0f);
								GroupBiNormalValue = FVector3f(0.0f);
							}
							if (!TriTangent.IsNearlyZero(SMALL_NUMBER) && !TriTangent.ContainsNaN())
							{
								GroupTangentValue += TriTangent;
							}
							if (!TriBinormal.IsNearlyZero(SMALL_NUMBER) && !TriBinormal.ContainsNaN())
							{
								GroupBiNormalValue += TriBinormal;
							}
						}
					}

					//////////////////////////////////////////////////////////////////////////
					//Apply the group to the Mesh
					GroupNormal.Normalize();
					if (!bComputeTangentWithMikkTSpace)
					{
						for (auto& Kvp : GroupTangent)
						{
							Kvp.Value.Normalize();
						}
						for (auto& Kvp : GroupBiNormal)
						{
							Kvp.Value.Normalize();
						}
					}
					//Apply the average NTB on all Vertex instance
					for (const FVertexInstanceID& VertexInstanceID : VertexInstanceInGroup)
					{
						const FVector2f& VertexUV = !VertexUVs.IsEmpty() ? VertexUVs[VertexInstanceID] : FVector2f::ZeroVector;

						if (VertexNormals[VertexInstanceID].IsNearlyZero(SMALL_NUMBER))
						{
							VertexNormals[VertexInstanceID] = GroupNormal;
						}

						//If we are not computing the tangent with MikkTSpace, make sure the tangents are valid.
						if (!bComputeTangentWithMikkTSpace)
						{
							//Avoid changing the original group value
							FVector3f GroupTangentValue = GroupTangent[VertexUV];
							FVector3f GroupBiNormalValue = GroupBiNormal[VertexUV];

							if (!VertexTangents[VertexInstanceID].IsNearlyZero(SMALL_NUMBER))
							{
								GroupTangentValue = VertexTangents[VertexInstanceID];
							}
							FVector3f BiNormal(0.0f);
							const FVector3f& VertexNormal(VertexNormals[VertexInstanceID]);
							if (!VertexNormal.IsNearlyZero(SMALL_NUMBER) && !VertexTangents[VertexInstanceID].IsNearlyZero(SMALL_NUMBER))
							{
								BiNormal = FVector3f::CrossProduct(VertexNormal, VertexTangents[VertexInstanceID]).GetSafeNormal() * VertexBinormalSigns[VertexInstanceID];
							}
							if (!BiNormal.IsNearlyZero(SMALL_NUMBER))
							{
								GroupBiNormalValue = BiNormal;
							}
							// Gram-Schmidt orthogonalization
							GroupBiNormalValue -= GroupTangentValue * (GroupTangentValue | GroupBiNormalValue);
							GroupBiNormalValue.Normalize();

							GroupTangentValue -= VertexNormal * (VertexNormal | GroupTangentValue);
							GroupTangentValue.Normalize();

							GroupBiNormalValue -= VertexNormal * (VertexNormal | GroupBiNormalValue);
							GroupBiNormalValue.Normalize();
							//Set the value
							VertexTangents[VertexInstanceID] = GroupTangentValue;
							//If the BiNormal is zero set the sign to 1.0f, inlining GetBasisDeterminantSign() to avoid depending on RenderCore.
							VertexBinormalSigns[VertexInstanceID] = FMatrix44f(GroupTangentValue, GroupBiNormalValue, VertexNormal, FVector3f::ZeroVector).Determinant() < 0 ? -1.0f : +1.0f;
						}
					}
				}
			}
		}
	);

	if (bForceComputeTangent && bComputeTangentWithMikkTSpace)
	{
		ComputeMikktTangents(MeshDescription, EnumHasAnyFlags(ComputeNTBsOptions, EComputeNTBsFlags::IgnoreDegenerateTriangles));
	}
}

#if WITH_MIKKTSPACE
namespace MeshDescriptionMikktSpaceInterface
{
	struct FMeshDescriptionCachedData
	{
		int32 NumTriangles;
		TArrayView<const FVertexID> TriangleVertexIDs;
		TArrayView<const FVertexInstanceID> TriangleVertexInstanceIDs;
		TArrayView<const FVector3f> VertexPositions;
		TArrayView<const FVector3f> VertexInstanceNormals;
		TArrayView<const FVector2f> VertexInstanceUVs;
		TArrayView<FVector3f> VertexInstanceTangents;
		TArrayView<float> VertexInstanceBinormalSigns;
	};

	int MikkGetNumFaces(const SMikkTSpaceContext* Context)
	{
		FMeshDescriptionCachedData* UserData = (FMeshDescriptionCachedData*)(Context->m_pUserData);
		return UserData->NumTriangles;
	}

	int MikkGetNumVertsOfFace(const SMikkTSpaceContext* Context, const int FaceIdx)
	{
		FMeshDescriptionCachedData* UserData = (FMeshDescriptionCachedData*)(Context->m_pUserData);
		return 3;
	}

	void MikkGetPosition(const SMikkTSpaceContext* Context, float Position[3], const int FaceIdx, const int VertIdx)
	{
		FMeshDescriptionCachedData* UserData = (FMeshDescriptionCachedData*)(Context->m_pUserData);
		const FVector3f& VertexPosition = UserData->VertexPositions[UserData->TriangleVertexIDs[FaceIdx * 3 + VertIdx]];
		Position[0] = VertexPosition.X;
		Position[1] = VertexPosition.Y;
		Position[2] = VertexPosition.Z;
	}

	void MikkGetNormal(const SMikkTSpaceContext* Context, float Normal[3], const int FaceIdx, const int VertIdx)
	{
		FMeshDescriptionCachedData* UserData = (FMeshDescriptionCachedData*)(Context->m_pUserData);
		const FVector3f& VertexNormal = UserData->VertexInstanceNormals[UserData->TriangleVertexInstanceIDs[FaceIdx * 3 + VertIdx]];
		Normal[0] = VertexNormal.X;
		Normal[1] = VertexNormal.Y;
		Normal[2] = VertexNormal.Z;
	}

	void MikkSetTSpaceBasic(const SMikkTSpaceContext* Context, const float Tangent[3], const float BitangentSign, const int FaceIdx, const int VertIdx)
	{
		FMeshDescriptionCachedData* UserData = (FMeshDescriptionCachedData*)(Context->m_pUserData);
		const FVertexInstanceID VertexInstanceID = UserData->TriangleVertexInstanceIDs[FaceIdx * 3 + VertIdx];
		UserData->VertexInstanceTangents[VertexInstanceID] = FVector3f(Tangent[0], Tangent[1], Tangent[2]);
		UserData->VertexInstanceBinormalSigns[VertexInstanceID] = -BitangentSign;
	}

	void MikkGetTexCoord(const SMikkTSpaceContext* Context, float UV[2], const int FaceIdx, const int VertIdx)
	{
		FMeshDescriptionCachedData* UserData = (FMeshDescriptionCachedData*)(Context->m_pUserData);
		const FVector2f& TexCoord = UserData->VertexInstanceUVs[UserData->TriangleVertexInstanceIDs[FaceIdx * 3 + VertIdx]];
		UV[0] = TexCoord.X;
		UV[1] = TexCoord.Y;
	}
}
#endif //#WITH_MIKKTSPACE

void FStaticMeshOperations::ComputeMikktTangents(FMeshDescription& MeshDescription, bool bIgnoreDegenerateTriangles)
{
#if WITH_MIKKTSPACE
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshOperations::ComputeMikktTangents);

	// The Mikkt interface does not handle properly polygon array with 'holes'
	// Compact mesh description if this is the case
	if (MeshDescription.NeedsCompact())
	{
		FElementIDRemappings Remappings;
		MeshDescription.Compact(Remappings);
	}

	int32 NumTriangles = MeshDescription.Triangles().Num();
	if (NumTriangles == 0)
	{
		return; // nothing to compute
	}

	// we can use mikktspace to calculate the tangents
	SMikkTSpaceInterface MikkTInterface;
	MikkTInterface.m_getNormal = MeshDescriptionMikktSpaceInterface::MikkGetNormal;
	MikkTInterface.m_getNumFaces = MeshDescriptionMikktSpaceInterface::MikkGetNumFaces;
	MikkTInterface.m_getNumVerticesOfFace = MeshDescriptionMikktSpaceInterface::MikkGetNumVertsOfFace;
	MikkTInterface.m_getPosition = MeshDescriptionMikktSpaceInterface::MikkGetPosition;
	MikkTInterface.m_getTexCoord = MeshDescriptionMikktSpaceInterface::MikkGetTexCoord;
	MikkTInterface.m_setTSpaceBasic = MeshDescriptionMikktSpaceInterface::MikkSetTSpaceBasic;
	MikkTInterface.m_setTSpace = nullptr;

	MeshDescriptionMikktSpaceInterface::FMeshDescriptionCachedData UserData;
	UserData.NumTriangles = MeshDescription.Triangles().Num();

	FStaticMeshAttributes Attributes(MeshDescription);
	UserData.TriangleVertexIDs = Attributes.GetTriangleVertexIndices().GetRawArray();
	UserData.TriangleVertexInstanceIDs = Attributes.GetTriangleVertexInstanceIndices().GetRawArray();
	UserData.VertexPositions = Attributes.GetVertexPositions().GetRawArray();
	UserData.VertexInstanceUVs = Attributes.GetVertexInstanceUVs().GetRawArray(0);
	UserData.VertexInstanceNormals = Attributes.GetVertexInstanceNormals().GetRawArray();
	UserData.VertexInstanceTangents = Attributes.GetVertexInstanceTangents().GetRawArray();
	UserData.VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns().GetRawArray();
	
	SMikkTSpaceContext MikkTContext;
	MikkTContext.m_pInterface = &MikkTInterface;
	MikkTContext.m_pUserData = (void*)(&UserData);
	MikkTContext.m_bIgnoreDegenerates = bIgnoreDegenerateTriangles;
	genTangSpaceDefault(&MikkTContext);
#else
	ensureMsgf(false, TEXT("MikkTSpace tangent generation is not supported on this platform."));
#endif //WITH_MIKKTSPACE
}

void FStaticMeshOperations::FindOverlappingCorners(FOverlappingCorners& OutOverlappingCorners, const FMeshDescription& MeshDescription, float ComparisonThreshold)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshOperations::FindOverlappingCorners);

	// @todo: this should be shared with FOverlappingCorners

	const FVertexInstanceArray& VertexInstanceArray = MeshDescription.VertexInstances();
	const FVertexArray& VertexArray = MeshDescription.Vertices();

	int32 NumWedges = 3 * MeshDescription.Triangles().Num();

	// Empty the old data and reserve space for new
	OutOverlappingCorners.Init(NumWedges);

	// Create a list of vertex Z/index pairs
	TArray<MeshDescriptionOperationNamespace::FIndexAndZ> VertIndexAndZ;
	VertIndexAndZ.Reserve(NumWedges);

	TVertexAttributesConstRef<FVector3f> VertexPositions = MeshDescription.GetVertexPositions();

	int32 WedgeIndex = 0;
	for (const FPolygonID PolygonID : MeshDescription.Polygons().GetElementIDs())
	{
		TArrayView<const FTriangleID> TriangleIDs = MeshDescription.GetPolygonTriangles(PolygonID);
		for (const FTriangleID& TriangleID : TriangleIDs)
		{
			for (int32 Corner = 0; Corner < 3; ++Corner)
			{
				const FVertexInstanceID VertexInstanceID = MeshDescription.GetTriangleVertexInstance(TriangleID, Corner);
				VertIndexAndZ.Emplace(WedgeIndex, VertexPositions[MeshDescription.GetVertexInstanceVertex(VertexInstanceID)]);
				++WedgeIndex;
			}
		}
	}

	// Sort the vertices by z value
	VertIndexAndZ.Sort(MeshDescriptionOperationNamespace::FCompareIndexAndZ());

	// Search for duplicates, quickly!
	for (int32 i = 0; i < VertIndexAndZ.Num(); i++)
	{
		// only need to search forward, since we add pairs both ways
		for (int32 j = i + 1; j < VertIndexAndZ.Num(); j++)
		{
			if (FMath::Abs(VertIndexAndZ[j].Z - VertIndexAndZ[i].Z) > ComparisonThreshold)
				break; // can't be any more dups

			const FVector3f& PositionA = *(VertIndexAndZ[i].OriginalVector);
			const FVector3f& PositionB = *(VertIndexAndZ[j].OriginalVector);

			if (PositionA.Equals(PositionB, ComparisonThreshold))
			{
				OutOverlappingCorners.Add(VertIndexAndZ[i].Index, VertIndexAndZ[j].Index);
				OutOverlappingCorners.Add(VertIndexAndZ[j].Index, VertIndexAndZ[i].Index);
			}
		}
	}

	OutOverlappingCorners.FinishAdding();
}

struct FLayoutUVMeshDescriptionView final : FLayoutUV::IMeshView
{
	FMeshDescription& MeshDescription;
	TVertexAttributesConstRef<FVector3f> Positions;
	TVertexInstanceAttributesConstRef<FVector3f> Normals;
	TVertexInstanceAttributesRef<FVector2f> TexCoords;

	const uint32 SrcChannel;
	const uint32 DstChannel;

	uint32 NumIndices = 0;
	TArray<int32> RemapVerts;
	TArray<FVector2f> FlattenedTexCoords;

	FLayoutUVMeshDescriptionView(FMeshDescription& InMeshDescription, uint32 InSrcChannel, uint32 InDstChannel)
		: MeshDescription(InMeshDescription)
		, SrcChannel(InSrcChannel)
		, DstChannel(InDstChannel)
	{
		FStaticMeshAttributes Attributes(InMeshDescription);
		Positions = Attributes.GetVertexPositions();
		Normals = Attributes.GetVertexInstanceNormals();
		TexCoords = Attributes.GetVertexInstanceUVs();

		uint32 NumTris = MeshDescription.Triangles().Num();

		NumIndices = NumTris * 3;

		FlattenedTexCoords.SetNumUninitialized(NumIndices);
		RemapVerts.SetNumUninitialized(NumIndices);

		int32 WedgeIndex = 0;

		for (const FPolygonID PolygonID : MeshDescription.Polygons().GetElementIDs())
		{
			TArrayView<const FTriangleID> TriangleIDs = MeshDescription.GetPolygonTriangles(PolygonID);
			for (const FTriangleID& TriangleID : TriangleIDs)
			{
				for (int32 Corner = 0; Corner < 3; ++Corner)
				{
					const FVertexInstanceID VertexInstanceID = MeshDescription.GetTriangleVertexInstance(TriangleID, Corner);

					FlattenedTexCoords[WedgeIndex] = TexCoords.Get(VertexInstanceID, SrcChannel);
					RemapVerts[WedgeIndex] = VertexInstanceID.GetValue();
					++WedgeIndex;
				}
			}
		}
	}

	uint32 GetNumIndices() const override { return NumIndices; }

	FVector3f GetPosition(uint32 Index) const override
	{
		FVertexInstanceID VertexInstanceID(RemapVerts[Index]);
		FVertexID VertexID = MeshDescription.GetVertexInstanceVertex(VertexInstanceID);
		return Positions[VertexID];
	}

	FVector3f GetNormal(uint32 Index) const override
	{
		FVertexInstanceID VertexInstanceID(RemapVerts[Index]);
		return Normals[VertexInstanceID];
	}

	FVector2f GetInputTexcoord(uint32 Index) const override
	{
		return FlattenedTexCoords[Index];
	}

	void InitOutputTexcoords(uint32 Num) override
	{
		// If current DstChannel is out of range of the number of UVs defined by the mesh description, change the index count accordingly
		const uint32 NumUVs = TexCoords.GetNumChannels();
		if (DstChannel >= NumUVs)
		{
			TexCoords.SetNumChannels(DstChannel + 1);
			ensure(false);	// not expecting it to get here
		}
	}

	void SetOutputTexcoord(uint32 Index, const FVector2f& Value) override
	{
		const FVertexInstanceID VertexInstanceID(RemapVerts[Index]);
		TexCoords.Set(VertexInstanceID, DstChannel, Value);
	}
};

int32 FStaticMeshOperations::GetUVChartCount(FMeshDescription& MeshDescription, int32 SrcLightmapIndex, ELightmapUVVersion LightmapUVVersion, const FOverlappingCorners& OverlappingCorners)
{
	uint32 UnusedDstIndex = -1;
	FLayoutUVMeshDescriptionView MeshDescriptionView(MeshDescription, SrcLightmapIndex, UnusedDstIndex);
	FLayoutUV Packer(MeshDescriptionView);
	Packer.SetVersion(LightmapUVVersion);
	return Packer.FindCharts(OverlappingCorners);
}

bool FStaticMeshOperations::CreateLightMapUVLayout(FMeshDescription& MeshDescription,
	int32 SrcLightmapIndex,
	int32 DstLightmapIndex,
	int32 MinLightmapResolution,
	ELightmapUVVersion LightmapUVVersion,
	const FOverlappingCorners& OverlappingCorners)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshOperations::CreateLightMapUVLayout)

		FLayoutUVMeshDescriptionView MeshDescriptionView(MeshDescription, SrcLightmapIndex, DstLightmapIndex);
	FLayoutUV Packer(MeshDescriptionView);
	Packer.SetVersion(LightmapUVVersion);

	if (LightmapUVVersion >= ELightmapUVVersion::ForceLightmapPadding)
	{
		MinLightmapResolution -= 2;
	}

	Packer.FindCharts(OverlappingCorners);
	bool bPackSuccess = Packer.FindBestPacking(MinLightmapResolution);
	if (bPackSuccess)
	{
		Packer.CommitPackedUVs();
	}
	return bPackSuccess;
}

static bool GatherUniqueTriangles(const FMeshDescription& InMeshDescription, bool bMergeIdenticalMaterials, TArray<FTriangleID>& OutRemappedTriangles, TArray<FVertexInstanceID>* OutUniqueVerts, TArray<FTriangleID>* OutDuplicateTriangles)
{
	FStaticMeshConstAttributes Attributes(InMeshDescription);
	TVertexInstanceAttributesConstRef<FVector2f> TexCoords = Attributes.GetVertexInstanceUVs();
	TVertexInstanceAttributesConstRef<FVector4f> VertexColors = Attributes.GetVertexInstanceColors();

	int32 NumVertexInstances = InMeshDescription.VertexInstances().Num();
	int32 NumTriangles = InMeshDescription.Triangles().Num();

	OutRemappedTriangles.Reserve(NumTriangles);

	TMap<uint32, FTriangleID> UniqueTriangles;
	if (bMergeIdenticalMaterials)
	{
		UniqueTriangles.Reserve(NumTriangles);
	}

	if (OutUniqueVerts)
	{
		OutUniqueVerts->Reserve(NumVertexInstances);
	}

	if (OutDuplicateTriangles)
	{
		OutDuplicateTriangles->Reserve(NumTriangles);
	}

	// Compute an hash value per triangle, based on its UVs & vertices colors
	auto HashAttribute = [](FVertexInstanceID InVertexInstanceID, auto InAttributeArrayRef, int32& TriangleHash)
	{
		for (int32 Channel = 0; Channel < InAttributeArrayRef.GetNumChannels(); ++Channel)
		{
			for (const auto& Element : InAttributeArrayRef.GetArrayView(InVertexInstanceID, Channel))
			{
				TriangleHash = HashCombine(TriangleHash, GetTypeHash(Element));
			}
		}
	};

	for (const FTriangleID TriangleID : InMeshDescription.Triangles().GetElementIDs())
	{
		const FPolygonGroupID RefPolygonGroupID = InMeshDescription.GetTrianglePolygonGroup(TriangleID);
		TConstArrayView<const FVertexInstanceID> VertexInstancesIDs = InMeshDescription.GetTriangleVertexInstances(TriangleID);
		TConstArrayView<const FVertexID> VertexIDs = InMeshDescription.GetTriangleVertices(TriangleID);

		FTriangleID RemapTriangleID = TriangleID;

		bool bUnique = true;

		if (bMergeIdenticalMaterials)
		{
			int32 TriangleHash = GetTypeHash(RefPolygonGroupID);
			for (const FVertexInstanceID& VertexInstanceID : VertexInstancesIDs)
			{
				// Compute hash based on UVs & vertices colors
				HashAttribute(VertexInstanceID, TexCoords, TriangleHash);
				HashAttribute(VertexInstanceID, VertexColors, TriangleHash);
			}

			FTriangleID* UniqueTriangleIDPtr = UniqueTriangles.Find(TriangleHash);
			if (UniqueTriangleIDPtr != nullptr)
			{
				RemapTriangleID = *UniqueTriangleIDPtr;
				bUnique = false;

				if (OutDuplicateTriangles)
				{
					OutDuplicateTriangles->Add(TriangleID);
				}
			}
			else
			{
				UniqueTriangles.Add(TriangleHash, TriangleID);
			}
		}

		if (bUnique && OutUniqueVerts)
		{
			OutUniqueVerts->Append(VertexInstancesIDs);
		}

		OutRemappedTriangles.Add(RemapTriangleID);
	}

	const bool bPerformedRemapping = bMergeIdenticalMaterials && UniqueTriangles.Num() != OutRemappedTriangles.Num();
	return bPerformedRemapping;
}

template <typename TSrcUVs, typename TDstUVs>
static void CopyRemappedUVs(const FMeshDescription& InMeshDescription, const TArray<FTriangleID>& RemappedTriangles, const FElementIDRemappings* SrcElementIDRemappings, bool bCopyOnlyRemappedTrianglesUVs, const TSrcUVs& SrcUVs, TDstUVs& DstUVs)
{
	int32 RemappedTrianglesIdx = 0;
	for (const FTriangleID TriangleID : InMeshDescription.Triangles().GetElementIDs())
	{
		FTriangleID RemappedTriangleID = RemappedTriangles[RemappedTrianglesIdx];

		if (!bCopyOnlyRemappedTrianglesUVs || RemappedTriangleID != TriangleID)
		{
			if (SrcElementIDRemappings)
			{
				RemappedTriangleID = SrcElementIDRemappings->GetRemappedTriangleID(RemappedTriangleID);
			}

			TConstArrayView<const FVertexInstanceID> SrcVertexInstancesIDs = InMeshDescription.GetTriangleVertexInstances(RemappedTriangleID);
			TConstArrayView<const FVertexInstanceID> DstVertexInstancesIDs = InMeshDescription.GetTriangleVertexInstances(TriangleID);

			for (int32 i = 0; i < 3; i++)
			{
				DstUVs[DstVertexInstancesIDs[i]] = FVector2D(SrcUVs[SrcVertexInstancesIDs[i]]);
			}
		}

		RemappedTrianglesIdx++;
	}
}

// Mesh view that will expose only unique UVs if bMergeIdenticalMaterials is provided
struct FUniqueUVMeshDescriptionView final : FLayoutUV::IMeshView
{
	const FMeshDescription& MeshDescription;

	TVertexAttributesConstRef<FVector3f> Positions;
	TVertexInstanceAttributesConstRef<FVector3f> Normals;
	TVertexInstanceAttributesConstRef<FVector2f> TexCoords;

	TArray<FTriangleID>					RemapTriangles;
	TArray<FVertexInstanceID>			UniqueVerts;
	TArray<FVector2D>&					OutputTexCoords;
	bool								bMustRemap;

	FUniqueUVMeshDescriptionView(const FMeshDescription& InMeshDescription, bool bMergeIdenticalMaterials, TArray<FVector2D>& InOutTexCoords)
		: MeshDescription(InMeshDescription)
		, OutputTexCoords(InOutTexCoords)
	{
		FStaticMeshConstAttributes Attributes(MeshDescription);
		Positions = Attributes.GetVertexPositions();
		Normals = Attributes.GetVertexInstanceNormals();
		TexCoords = Attributes.GetVertexInstanceUVs();

		OutputTexCoords.SetNumZeroed(MeshDescription.VertexInstances().Num());

		bMustRemap = GatherUniqueTriangles(MeshDescription, bMergeIdenticalMaterials, RemapTriangles, &UniqueVerts, nullptr);
	}

	uint32 GetNumIndices() const override
	{
		return UniqueVerts.Num();
	}

	FVector3f GetPosition(uint32 Index) const override
	{
		FVertexID VertexID = MeshDescription.GetVertexInstanceVertex(UniqueVerts[Index]);
		return Positions[VertexID];
	}

	FVector3f GetNormal(uint32 Index) const override
	{
		return Normals[UniqueVerts[Index]];
	}

	FVector2f GetInputTexcoord(uint32 Index) const override
	{
		return TexCoords.Get(UniqueVerts[Index], 0);
	}

	void InitOutputTexcoords(uint32 Num) override
	{
		check(Num == UniqueVerts.Num());
	}

	void SetOutputTexcoord(uint32 Index, const FVector2f& Value) override
	{
		OutputTexCoords[UniqueVerts[Index]] = FVector2D(Value);
	}

	void ResolvePackedUVs()
	{
		if (bMustRemap)
		{
			CopyRemappedUVs(MeshDescription, RemapTriangles, nullptr, true, OutputTexCoords, OutputTexCoords);
		}
	}
};

bool FStaticMeshOperations::GenerateUniqueUVsForStaticMesh(const FMeshDescription& MeshDescription, int32 TextureResolution, bool bMergeIdenticalMaterials, TArray<FVector2D>& OutTexCoords)
{
	FGenerateUVOptions GenerateUVOptions;
	GenerateUVOptions.TextureResolution = TextureResolution;
	GenerateUVOptions.bMergeTrianglesWithIdenticalAttributes = bMergeIdenticalMaterials;
	GenerateUVOptions.UVMethod = EGenerateUVMethod::Legacy;

	return GenerateUV(MeshDescription, GenerateUVOptions, OutTexCoords);
}

bool FStaticMeshOperations::GenerateUV(const FMeshDescription& MeshDescription, const FGenerateUVOptions& GenerateUVOptions, TArray<FVector2D>& OutTexCoords)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshOperations::GenerateUniqueUVsForStaticMesh)

	TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = FStaticMeshConstAttributes(MeshDescription).GetVertexInstanceUVs();

	OutTexCoords.Reset();

	const bool bHasUVs = VertexInstanceUVs.GetNumElements() > 0;
#if WITH_EDITOR
	if (bHasUVs && GenerateUVOptions.UVMethod == EGenerateUVMethod::Legacy)
#else
	if (bHasUVs)
#endif
	{
		FUniqueUVMeshDescriptionView MeshDescriptionView(MeshDescription, GenerateUVOptions.bMergeTrianglesWithIdenticalAttributes, OutTexCoords);

		// Find overlapping corners for UV generator. Allow some threshold - this should not produce any error in a case if resulting
		// mesh will not merge these vertices.
		FOverlappingCorners OverlappingCorners(MeshDescriptionView, THRESH_POINTS_ARE_SAME);

		// Generate new UVs
		FLayoutUV Packer(MeshDescriptionView);
		int32 NumCharts = Packer.FindCharts(OverlappingCorners);

		// Scale down texture resolution to speed up UV generation time
		// Packing expects at least one texel per chart. This is the absolute minimum to generate valid UVs.
		const int32 PackingResolution = FMath::Clamp(GenerateUVOptions.TextureResolution / 4, 32, 512);
		const int32 AbsoluteMinResolution = 1 << FMath::CeilLogTwo(FMath::Sqrt((float)NumCharts));
		const int32 FinalPackingResolution = FMath::Max(PackingResolution, AbsoluteMinResolution);

		bool bPackSuccess = Packer.FindBestPacking(FinalPackingResolution);
		if (bPackSuccess)
		{
			Packer.CommitPackedUVs();
			MeshDescriptionView.ResolvePackedUVs();
		}
		else
		{
			OutTexCoords.Reset();
		}
	}
	
#if WITH_EDITOR
	// Missing/invalid UVs, use the AutoUV interface
	if (OutTexCoords.IsEmpty())
	{
		IGeometryProcessingInterfacesModule* GeomProcInterfaces = FModuleManager::Get().GetModulePtr<IGeometryProcessingInterfacesModule>("GeometryProcessingInterfaces");
		IGeometryProcessing_MeshAutoUV* MeshAutoUV = GeomProcInterfaces ? GeomProcInterfaces->GetMeshAutoUVImplementation() : nullptr;
		if (MeshAutoUV)
		{
		    FMeshDescription MeshCopy = MeshDescription;
			TArray<FTriangleID>	RemapTriangles;
			FElementIDRemappings ElementIDRemappings;

			bool bMustRemap = false;

			// Ensure we have properly setup TriangleUVs on our mesh
			int32 UVChannelCount = MeshCopy.VertexInstanceAttributes().GetAttributeChannelCount(MeshAttribute::VertexInstance::TextureCoordinate);
			MeshCopy.SetNumUVChannels(UVChannelCount);

			if (GenerateUVOptions.bMergeTrianglesWithIdenticalAttributes)
			{
				TArray<FTriangleID> DuplicateTriangles;

				bMustRemap = GatherUniqueTriangles(MeshDescription, true, RemapTriangles, nullptr, &DuplicateTriangles);
				if (bMustRemap)
				{
					MeshCopy.DeleteTriangles(DuplicateTriangles);
					MeshCopy.Compact(ElementIDRemappings);
				}
			}

			auto GetAutoUVMethod = [](EGenerateUVMethod GenerateUVMethod) -> IGeometryProcessing_MeshAutoUV::EAutoUVMethod
			{
				switch (GenerateUVMethod)
				{
				case EGenerateUVMethod::UVAtlas:	return IGeometryProcessing_MeshAutoUV::EAutoUVMethod::UVAtlas;
				case EGenerateUVMethod::XAtlas:		return IGeometryProcessing_MeshAutoUV::EAutoUVMethod::XAtlas;
				default:							return IGeometryProcessing_MeshAutoUV::EAutoUVMethod::PatchBuilder;
				}
			};
			    
		    IGeometryProcessing_MeshAutoUV::FOptions Options = MeshAutoUV->ConstructDefaultOptions();
			Options.Method = GetAutoUVMethod(GenerateUVOptions.UVMethod);

		    IGeometryProcessing_MeshAutoUV::FResults Results;
		    MeshAutoUV->GenerateUVs(MeshCopy, Options, Results);
    
		    if (Results.ResultCode == IGeometryProcessing_MeshAutoUV::EResultCode::Success)
		    {
			    TVertexInstanceAttributesConstRef<FVector2f> TexCoords;
			    FStaticMeshConstAttributes AttributesCopy(MeshCopy);
			    TexCoords = AttributesCopy.GetVertexInstanceUVs();

				OutTexCoords.SetNumUninitialized(MeshDescription.VertexInstances().Num());
    
				if (bMustRemap)
				{
					CopyRemappedUVs(MeshDescription, RemapTriangles, &ElementIDRemappings, false, TexCoords, OutTexCoords);
				}
				else
				{
					int32 VertexInstanceIndex = 0;
					for (const FVertexInstanceID VertexInstanceID : MeshCopy.VertexInstances().GetElementIDs())
					{
						OutTexCoords[VertexInstanceIndex] = FVector2D(TexCoords.Get(VertexInstanceID, 0));
						VertexInstanceIndex++;
					}
				}
		    }
		}
	}
#endif

	check(OutTexCoords.IsEmpty() || OutTexCoords.Num() == MeshDescription.VertexInstances().Num());

	return !OutTexCoords.IsEmpty();
}

bool FStaticMeshOperations::AddUVChannel(FMeshDescription& MeshDescription)
{
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);
	if (VertexInstanceUVs.GetNumChannels() >= MAX_MESH_TEXTURE_COORDS)
	{
		UE_LOGF(LogStaticMeshOperations, Error, "AddUVChannel: Cannot add UV channel. Maximum number of UV channels reached (%d).", MAX_MESH_TEXTURE_COORDS);
		return false;
	}

	VertexInstanceUVs.SetNumChannels(VertexInstanceUVs.GetNumChannels() + 1);
	return true;
}

bool FStaticMeshOperations::InsertUVChannel(FMeshDescription& MeshDescription, int32 UVChannelIndex)
{
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);
	if (UVChannelIndex < 0 || UVChannelIndex > VertexInstanceUVs.GetNumChannels())
	{
		UE_LOGF(LogStaticMeshOperations, Error, "InsertUVChannel: Cannot insert UV channel. Given UV channel index %d is out of bounds.", UVChannelIndex);
		return false;
	}

	if (VertexInstanceUVs.GetNumChannels() >= MAX_MESH_TEXTURE_COORDS)
	{
		UE_LOGF(LogStaticMeshOperations, Error, "InsertUVChannel: Cannot insert UV channel. Maximum number of UV channels reached (%d).", MAX_MESH_TEXTURE_COORDS);
		return false;
	}

	VertexInstanceUVs.InsertChannel(UVChannelIndex);
	return true;
}

bool FStaticMeshOperations::RemoveUVChannel(FMeshDescription& MeshDescription, int32 UVChannelIndex)
{
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate);
	if (VertexInstanceUVs.GetNumChannels() == 1)
	{
		UE_LOGF(LogStaticMeshOperations, Error, "RemoveUVChannel: Cannot remove UV channel. There must be at least one channel.");
		return false;
	}

	if (UVChannelIndex < 0 || UVChannelIndex >= VertexInstanceUVs.GetNumChannels())
	{
		UE_LOGF(LogStaticMeshOperations, Error, "RemoveUVChannel: Cannot remove UV channel. Given UV channel index %d is out of bounds.", UVChannelIndex);
		return false;
	}

	VertexInstanceUVs.RemoveChannel(UVChannelIndex);
	return true;
}

void FStaticMeshOperations::GeneratePlanarUV(const FMeshDescription& MeshDescription, const FUVMapParameters& Params, TMap<FVertexInstanceID, FVector2D>& OutTexCoords)
{
	// Project along X-axis (left view), UV along Z Y axes
	FVector U = FVector::UpVector;
	FVector V = FVector::RightVector;

	TMeshAttributesConstRef<FVertexID, FVector3f> VertexPositions = MeshDescription.GetVertexPositions();

	OutTexCoords.Reserve(MeshDescription.VertexInstances().Num());

	FVector Size(Params.Size * Params.Scale);
	FVector Offset = Params.Position - Size / 2.f;

	for (const FVertexInstanceID VertexInstanceID : MeshDescription.VertexInstances().GetElementIDs())
	{
		const FVertexID VertexID = MeshDescription.GetVertexInstanceVertex(VertexInstanceID);
		FVector Vertex(VertexPositions[VertexID]);

		// Apply the gizmo transforms
		Vertex = Params.Rotation.RotateVector(Vertex);
		Vertex -= Offset;
		Vertex /= Size;

		float UCoord = FVector::DotProduct(Vertex, U) * Params.UVTile.X;
		float VCoord = FVector::DotProduct(Vertex, V) * Params.UVTile.Y;
		OutTexCoords.Add(VertexInstanceID, FVector2D(UCoord, VCoord));
	}
}

void FStaticMeshOperations::GenerateCylindricalUV(FMeshDescription& MeshDescription, const FUVMapParameters& Params, TMap<FVertexInstanceID, FVector2D>& OutTexCoords)
{
	FVector3f Size(Params.Size * Params.Scale);	//LWC_TODO: Precision loss
	FVector3f Offset(Params.Position);	//LWC_TODO: Precision loss

	// Cylinder along X-axis, counterclockwise from -Y axis as seen from left view
	FVector3f V = FVector3f::ForwardVector;
	Offset.X -= Size.X / 2.f;

	TMeshAttributesConstRef<FVertexID, FVector3f> VertexPositions = MeshDescription.GetVertexPositions();

	OutTexCoords.Reserve(MeshDescription.VertexInstances().Num());

	const float AngleOffset = PI; // offset to get the same result as in 3dsmax

	for (const FVertexInstanceID VertexInstanceID : MeshDescription.VertexInstances().GetElementIDs())
	{
		const FVertexID VertexID = MeshDescription.GetVertexInstanceVertex(VertexInstanceID);
		FVector3f Vertex = VertexPositions[VertexID];

		// Apply the gizmo transforms
		Vertex = FVector3f(Params.Rotation.RotateVector(FVector3d(Vertex)));
		Vertex -= Offset;
		Vertex /= Size;

		float Angle = FMath::Atan2(Vertex.Z, Vertex.Y);

		Angle += AngleOffset;
		Angle *= Params.UVTile.X;

		float UCoord = Angle / (2 * PI);
		float VCoord = FVector3f::DotProduct(Vertex, V) * Params.UVTile.Y;

		OutTexCoords.Add(VertexInstanceID, FVector2D(UCoord, VCoord));
	}

	// Fix the UV coordinates for triangles at the seam where the angle wraps around
	for (const FPolygonID PolygonID : MeshDescription.Polygons().GetElementIDs())
	{
		TArray<FVertexInstanceID, TInlineAllocator<4>> VertexInstances = MeshDescription.GetPolygonVertexInstances<TInlineAllocator<4>>(PolygonID);
		int32 NumInstances = VertexInstances.Num();
		if (NumInstances >= 2)
		{
			for (int32 StartIndex = 0; StartIndex < NumInstances; ++StartIndex)
			{
				int32 EndIndex = StartIndex + 1;
				if (EndIndex >= NumInstances)
				{
					EndIndex = EndIndex % NumInstances;
				}

				const FVector2D& StartUV = OutTexCoords[VertexInstances[StartIndex]];
				FVector2D& EndUV = OutTexCoords[VertexInstances[EndIndex]];

				// TODO: Improve fix for UVTile other than 1
				float Threshold = 0.5f / Params.UVTile.X;
				if (FMath::Abs(EndUV.X - StartUV.X) > Threshold)
				{
					// Fix the U coordinate to get the texture go counterclockwise
					if (EndUV.X > Threshold)
					{
						if (EndUV.X >= 1.f)
						{
							EndUV.X -= 1.f;
						}
					}
					else
					{
						if (EndUV.X <= 0)
						{
							EndUV.X += 1.f;
						}
					}
				}
			}
		}
	}
}

void FStaticMeshOperations::GenerateBoxUV(const FMeshDescription& MeshDescription, const FUVMapParameters& Params, TMap<FVertexInstanceID, FVector2D>& OutTexCoords)
{
	FVector3f Size(Params.Size * Params.Scale);	//LWC_TODO: Precision loss
	FVector3f HalfSize = Size / 2.0f;

	TMeshAttributesConstRef<FVertexID, FVector3f> VertexPositions = MeshDescription.GetVertexPositions();

	OutTexCoords.Reserve(MeshDescription.VertexInstances().Num());

	// Setup the UVs such that the mapping is from top-left to bottom-right when viewed orthographically
	TArray<TPair<FVector3f, FVector3f>> PlaneUVs;
	PlaneUVs.Add(TPair<FVector3f, FVector3f>(FVector3f::ForwardVector, FVector3f::RightVector));	// Top view
	PlaneUVs.Add(TPair<FVector3f, FVector3f>(FVector3f::BackwardVector, FVector3f::RightVector));	// Bottom view
	PlaneUVs.Add(TPair<FVector3f, FVector3f>(FVector3f::ForwardVector, FVector3f::DownVector));		// Right view
	PlaneUVs.Add(TPair<FVector3f, FVector3f>(FVector3f::BackwardVector, FVector3f::DownVector));	// Left view
	PlaneUVs.Add(TPair<FVector3f, FVector3f>(FVector3f::LeftVector, FVector3f::DownVector));		// Front view
	PlaneUVs.Add(TPair<FVector3f, FVector3f>(FVector3f::RightVector, FVector3f::DownVector));		// Back view

	TArray<FPlane4f> BoxPlanes;
	const FVector3f Center(Params.Position);	//LWC_TODO: Precision loss

	BoxPlanes.Add(FPlane4f(Center + FVector3f(0, 0, HalfSize.Z), FVector3f::UpVector));		// Top plane
	BoxPlanes.Add(FPlane4f(Center - FVector3f(0, 0, HalfSize.Z), FVector3f::DownVector));		// Bottom plane
	BoxPlanes.Add(FPlane4f(Center + FVector3f(0, HalfSize.Y, 0), FVector3f::RightVector));	// Right plane
	BoxPlanes.Add(FPlane4f(Center - FVector3f(0, HalfSize.Y, 0), FVector3f::LeftVector));		// Left plane
	BoxPlanes.Add(FPlane4f(Center + FVector3f(HalfSize.X, 0, 0), FVector3f::ForwardVector));	// Front plane
	BoxPlanes.Add(FPlane4f(Center - FVector3f(HalfSize.X, 0, 0), FVector3f::BackwardVector));	// Back plane

	// For each polygon, find the box plane that best matches the polygon normal
	for (const FTriangleID TriangleID : MeshDescription.Triangles().GetElementIDs())
	{
		TArrayView<const FVertexID> Vertices = MeshDescription.GetTriangleVertices(TriangleID);
		TArrayView<const FVertexInstanceID> VertexInstances = MeshDescription.GetTriangleVertexInstances(TriangleID);

		FVector3f Vertex0 = VertexPositions[Vertices[0]];
		FVector3f Vertex1 = VertexPositions[Vertices[1]];
		FVector3f Vertex2 = VertexPositions[Vertices[2]];

		FPlane4f PolygonPlane(Vertex0, Vertex2, Vertex1);

		// Find the box plane that is most aligned with the polygon plane
		// TODO: Also take the distance between the planes into consideration
		float MaxProj = 0.f;
		int32 BestPlaneIndex = 0;
		for (int32 Index = 0; Index < BoxPlanes.Num(); ++Index)
		{
			float Proj = FVector3f::DotProduct(BoxPlanes[Index], PolygonPlane);
			if (Proj > MaxProj)
			{
				MaxProj = Proj;
				BestPlaneIndex = Index;
			}
		}

		FVector3f U = PlaneUVs[BestPlaneIndex].Key;
		FVector3f V = PlaneUVs[BestPlaneIndex].Value;
		FVector3f Offset = FVector3f(Params.Position) - HalfSize * (U + V);

		for (const FVertexInstanceID& VertexInstanceID : VertexInstances)
		{
			const FVertexID VertexID = MeshDescription.GetVertexInstanceVertex(VertexInstanceID);
			FVector3f Vertex = VertexPositions[VertexID];

			// Apply the gizmo transforms
			Vertex = FVector3f(Params.Rotation.RotateVector(FVector3d(Vertex)));
			Vertex -= Offset;

			// Normalize coordinates
			Vertex.X = FMath::IsNearlyZero(Size.X) ? 0.0f : Vertex.X / Size.X;
			Vertex.Y = FMath::IsNearlyZero(Size.Y) ? 0.0f : Vertex.Y / Size.Y;
			Vertex.Z = FMath::IsNearlyZero(Size.Z) ? 0.0f : Vertex.Z / Size.Z;

			float UCoord = FVector3f::DotProduct(Vertex, U) * Params.UVTile.X;
			float VCoord = FVector3f::DotProduct(Vertex, V) * Params.UVTile.Y;

			OutTexCoords.Add(VertexInstanceID, FVector2D(UCoord, VCoord));
		}
	}
}

void FStaticMeshOperations::SwapPolygonPolygonGroup(FMeshDescription& MeshDescription, int32 SectionIndex, int32 TriangleIndexStart, int32 TriangleIndexEnd, bool bRemoveEmptyPolygonGroup)
{
	int32 TriangleIndex = 0;
	TPolygonGroupAttributesRef<FName> PolygonGroupNames = MeshDescription.PolygonGroupAttributes().GetAttributesRef<FName>(MeshAttribute::PolygonGroup::ImportedMaterialSlotName);

	FPolygonGroupID TargetPolygonGroupID(SectionIndex);
	if (!bRemoveEmptyPolygonGroup)
	{
		while (!MeshDescription.PolygonGroups().IsValid(TargetPolygonGroupID))
		{
			TargetPolygonGroupID = MeshDescription.CreatePolygonGroup();
			PolygonGroupNames[TargetPolygonGroupID] = FName(*(TEXT("SwapPolygonMaterialSlotName_") + FString::FromInt(TargetPolygonGroupID.GetValue())));
			TargetPolygonGroupID = FPolygonGroupID(SectionIndex);
		}
	}
	else
	{
		//This will not follow the SectionIndex value if the value is greater then the number of section (do not use this when merging meshes)
		if (!MeshDescription.PolygonGroups().IsValid(TargetPolygonGroupID))
		{
			TargetPolygonGroupID = MeshDescription.CreatePolygonGroup();
			PolygonGroupNames[TargetPolygonGroupID] = FName(*(TEXT("SwapPolygonMaterialSlotName_") + FString::FromInt(TargetPolygonGroupID.GetValue())));
		}
	}

	for (const FPolygonID PolygonID : MeshDescription.Polygons().GetElementIDs())
	{
		int32 TriangleCount = MeshDescription.GetPolygonTriangles(PolygonID).Num();
		if (TriangleIndex >= TriangleIndexStart && TriangleIndex < TriangleIndexEnd)
		{
			check(TriangleIndex + (TriangleCount - 1) < TriangleIndexEnd);
			FPolygonGroupID OldpolygonGroupID = MeshDescription.GetPolygonPolygonGroup(PolygonID);
			if (OldpolygonGroupID != TargetPolygonGroupID)
			{
				MeshDescription.SetPolygonPolygonGroup(PolygonID, TargetPolygonGroupID);
				if (bRemoveEmptyPolygonGroup && MeshDescription.GetPolygonGroupPolygonIDs(OldpolygonGroupID).Num() < 1)
				{
					MeshDescription.DeletePolygonGroup(OldpolygonGroupID);
				}
			}
		}
		TriangleIndex += TriangleCount;
	}
}

bool FStaticMeshOperations::HasVertexColor(const FMeshDescription& MeshDescription)
{
	TVertexInstanceAttributesConstRef<FVector4f> VertexInstanceColors = MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector4f>(MeshAttribute::VertexInstance::Color);
	bool bHasVertexColor = false;
	FVector4f WhiteColor(FLinearColor::White);
	for (const FVertexInstanceID VertexInstanceID : MeshDescription.VertexInstances().GetElementIDs())
	{
		if (VertexInstanceColors[VertexInstanceID] != WhiteColor)
		{
			bHasVertexColor = true;
			break;
		}
	}
	return bHasVertexColor;
}

void FStaticMeshOperations::BuildWeldedVertexIDRemap(const FMeshDescription& MeshDescription, const float WeldingThreshold, TMap<FVertexID, FVertexID>& OutVertexIDRemap)
{
	TVertexAttributesConstRef<FVector3f> VertexPositions = MeshDescription.GetVertexPositions();

	int32 NumVertex = MeshDescription.Vertices().Num();
	OutVertexIDRemap.Reserve(NumVertex);

	// Create a list of vertex Z/index pairs
	TArray<MeshDescriptionOperationNamespace::FIndexAndZ> VertIndexAndZ;
	VertIndexAndZ.Reserve(NumVertex);

	for (const FVertexID VertexID : MeshDescription.Vertices().GetElementIDs())
	{
		VertIndexAndZ.Emplace(VertexID.GetValue(), VertexPositions[VertexID]);
	}

	// Sort the vertices by z value
	VertIndexAndZ.Sort(MeshDescriptionOperationNamespace::FCompareIndexAndZ());

	// Search for duplicates, quickly!
	for (int32 i = 0; i < VertIndexAndZ.Num(); i++)
	{
		FVertexID Index_i = FVertexID(VertIndexAndZ[i].Index);
		if (OutVertexIDRemap.Contains(Index_i))
		{
			continue;
		}
		OutVertexIDRemap.FindOrAdd(Index_i) = Index_i;
		// only need to search forward, since we add pairs both ways
		for (int32 j = i + 1; j < VertIndexAndZ.Num(); j++)
		{
			if (FMath::Abs(VertIndexAndZ[j].Z - VertIndexAndZ[i].Z) > WeldingThreshold)
				break; // can't be any more dups

			const FVector3f& PositionA = *(VertIndexAndZ[i].OriginalVector);
			const FVector3f& PositionB = *(VertIndexAndZ[j].OriginalVector);

			if (PositionA.Equals(PositionB, WeldingThreshold))
			{
				OutVertexIDRemap.FindOrAdd(FVertexID(VertIndexAndZ[j].Index)) = Index_i;
			}
		}
	}
}

FSHAHash FStaticMeshOperations::ComputeSHAHash(const FMeshDescription& MeshDescription, bool bSkipTransientAttributes)
{
	FSHA1 HashState;
	TArray< FName > AttributesNames;

	auto HashAttributeSet = [&AttributesNames, &HashState, bSkipTransientAttributes](const auto& AttributeSet)
	{
		AttributesNames.Reset();

		if (!bSkipTransientAttributes)
		{
			AttributeSet.GetAttributeNames(AttributesNames);
		}
		else
		{
			AttributeSet.ForEach([&AttributesNames](const FName AttributeName, auto AttributesRef)
			{
				bool bIsTransient = (AttributesRef.GetFlags() & EMeshAttributeFlags::Transient) != EMeshAttributeFlags::None;
				if (!bIsTransient)
				{
					AttributesNames.Add(AttributeName);
				}
			});
		}

		AttributesNames.Sort(FNameLexicalLess());

		for (FName AttributeName : AttributesNames)
		{
			uint32 AttributeHash = AttributeSet.GetHash(AttributeName);
			HashState.Update((uint8*)&AttributeHash, sizeof(AttributeHash));
		}
	};

	HashAttributeSet(MeshDescription.VertexAttributes());
	HashAttributeSet(MeshDescription.VertexInstanceAttributes());
	HashAttributeSet(MeshDescription.EdgeAttributes());
	HashAttributeSet(MeshDescription.PolygonAttributes());
	HashAttributeSet(MeshDescription.PolygonGroupAttributes());

	FSHAHash OutHash;

	HashState.Final();
	HashState.GetHash(&OutHash.Hash[0]);

	return OutHash;
}

void FStaticMeshOperations::FlipPolygons(FMeshDescription& MeshDescription)
{
	TSet<FVertexInstanceID> VertexInstanceIDs;
	for (const FTriangleID TriangleID : MeshDescription.Triangles().GetElementIDs())
	{
		TArrayView<const FVertexInstanceID> TriVertInstances = MeshDescription.GetTriangleVertexInstances(TriangleID);
		for (const FVertexInstanceID TriVertInstance : TriVertInstances)
		{
			VertexInstanceIDs.Add(TriVertInstance);
		}
		MeshDescription.ReverseTriangleFacing(TriangleID);
	}

	// Flip tangents and normals
	TVertexInstanceAttributesRef<FVector3f> VertexNormals = MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Normal);
	TVertexInstanceAttributesRef<FVector3f> VertexTangents = MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Tangent);

	for (const FVertexInstanceID VertexInstanceID : VertexInstanceIDs)
	{
		// Just reverse the sign of the normals/tangents; note that since binormals are the cross product of normal with tangent, they are left untouched
		VertexNormals[VertexInstanceID] *= -1.0f;
		VertexTangents[VertexInstanceID] *= -1.0f;
	}
}

void FStaticMeshOperations::ApplyTransform(FMeshDescription& MeshDescription, const FTransform& Transform, bool bApplyCorrectNormalTransform)
{
	ApplyTransform(MeshDescription, Transform.ToMatrixWithScale(), bApplyCorrectNormalTransform);
}

void FStaticMeshOperations::ApplyTransform(FMeshDescription& MeshDescription, const FMatrix& Transform, bool bApplyCorrectNormalTransform)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshOperations::ApplyTransform)

	TVertexAttributesRef<FVector3f> VertexPositions = MeshDescription.VertexAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Vertex::Position);
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Normal);
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Tangent);
	TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = MeshDescription.VertexInstanceAttributes().GetAttributesRef<float>(MeshAttribute::VertexInstance::BinormalSign);

	for (const FVertexID VertexID : MeshDescription.Vertices().GetElementIDs())
	{
		VertexPositions[VertexID] = FVector4f(Transform.TransformPosition(FVector3d(VertexPositions[VertexID])));
	}

	const bool bIsMirrored = Transform.Determinant() < 0.f;
	const float MulBy = bIsMirrored ? -1.f : 1.f;

	FMatrix NormalsTransform, TangentsTransform;
	if (bApplyCorrectNormalTransform)
	{
		// Note: Assuming we'll normalize after, transforming by the transpose-adjoint * the sign of the determinant
		// is equivalent to transforming by the inverse transpose; ref: TMatrix::TransformByUsingAdjointT
		NormalsTransform = Transform.TransposeAdjoint() * (double)MulBy;
		// Note: Tangents *do not* transform by the Transform's inverse transpose, just by the Transform
		TangentsTransform = Transform;
	}
	else // match UE renderer
	{
		// UE's renderer transforms normals and tangents without scale (as in FTransform::TransformVectorNoScale)
		NormalsTransform = Transform;
		NormalsTransform.RemoveScaling();
		TangentsTransform = NormalsTransform;
	}

	for (const FVertexInstanceID VertexInstanceID : MeshDescription.VertexInstances().GetElementIDs())
	{
		FVector3f Tangent = VertexInstanceTangents[VertexInstanceID];
		FVector3f Normal = VertexInstanceNormals[VertexInstanceID];

		VertexInstanceTangents[VertexInstanceID] = (FVector3f)FVector(TangentsTransform.TransformVector((FVector)Tangent).GetSafeNormal());
		VertexInstanceNormals[VertexInstanceID] = (FVector3f)FVector(NormalsTransform.TransformVector((FVector)Normal).GetSafeNormal());

		float BinormalSign = VertexInstanceBinormalSigns[VertexInstanceID];
		VertexInstanceBinormalSigns[VertexInstanceID] = BinormalSign * MulBy;
	}

	if (bIsMirrored)
	{
		MeshDescription.ReverseAllPolygonFacing();
	}
}

namespace UE::Private
{
	class FPrivateVertexInfo
	{
	public:
		FVector3f			Position;
		FVector3f			Normal;
		FVector3f			Tangents[2];
		FLinearColor		Color;
		FVector2f			TexCoords[MAX_MESH_TEXTURE_COORDS_MD];

		void Validate()
		{
			Normal.Normalize();
			Tangents[0] -= (Tangents[0] | Normal) * Normal;
			Tangents[0].Normalize();
			Tangents[1] -= (Tangents[1] | Normal) * Normal;
			Tangents[1] -= (Tangents[1] | Tangents[0]) * Tangents[0];
			Tangents[1].Normalize();
			Color = Color.GetClamped();
		}

		bool Equals(const FPrivateVertexInfo& Other) const
		{
			constexpr float UVEpsilon = 1.0f / 1024.0f;
			if (!Position.Equals(Other.Position, UE_THRESH_POINTS_ARE_SAME) ||
				!Tangents[0].Equals(Other.Tangents[0], UE_THRESH_NORMALS_ARE_SAME) ||
				!Tangents[1].Equals(Other.Tangents[1], UE_THRESH_NORMALS_ARE_SAME) ||
				!Normal.Equals(Other.Normal, UE_THRESH_NORMALS_ARE_SAME) ||
				!Color.Equals(Other.Color))
			{
				return false;
			}

			// UVs
			for (int32 UVIndex = 0; UVIndex < MAX_MESH_TEXTURE_COORDS_MD; UVIndex++)
			{
				if (!TexCoords[UVIndex].Equals(Other.TexCoords[UVIndex], UVEpsilon))
				{
					return false;
				}
			}

			return true;
		}
	};
}

int32 FStaticMeshOperations::GetUniqueVertexCount(const FMeshDescription& MeshDescription)
{
	FOverlappingCorners OverlappingCorners;
	FStaticMeshOperations::FindOverlappingCorners(OverlappingCorners, MeshDescription, UE_THRESH_POINTS_ARE_SAME);
	return GetUniqueVertexCount(MeshDescription, OverlappingCorners);
}

int32 FStaticMeshOperations::GetUniqueVertexCount(const FMeshDescription& MeshDescription, const FOverlappingCorners& OverlappingCorners)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FStaticMeshOperations::GetUniqueVertexCount);

	constexpr uint32 NumTexCoords = MAX_MESH_TEXTURE_COORDS_MD;
	TArray< UE::Private::FPrivateVertexInfo > Verts;
	Verts.Reserve(MeshDescription.Vertices().Num());
	TMap< int32, int32 > VertsMap;
	int32 NumFaces = MeshDescription.Triangles().Num();
	int32 NumWedges = NumFaces * 3;
	const FStaticMeshConstAttributes MeshAttribute(MeshDescription);
	TVertexAttributesConstRef<FVector3f> VertexPositions = MeshAttribute.GetVertexPositions();
	TVertexInstanceAttributesConstRef<FVector3f> VertexNormals = MeshAttribute.GetVertexInstanceNormals();
	TVertexInstanceAttributesConstRef<FVector3f> VertexTangents = MeshAttribute.GetVertexInstanceTangents();
	TVertexInstanceAttributesConstRef<float> VertexBinormalSigns = MeshAttribute.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesConstRef<FVector4f> VertexColors = MeshAttribute.GetVertexInstanceColors();
	TVertexInstanceAttributesConstRef<FVector2f> VertexUVs = MeshAttribute.GetVertexInstanceUVs();

	int32 WedgeIndex = 0;
	for (const FTriangleID TriangleID : MeshDescription.Triangles().GetElementIDs())
	{
		TArrayView<const FVertexID> VertexIDs = MeshDescription.GetTriangleVertices(TriangleID);

		FVector3f CornerPositions[3];
		for (int32 TriVert = 0; TriVert < 3; ++TriVert)
		{
			const FVertexID VertexID = VertexIDs[TriVert];
			CornerPositions[TriVert] = VertexPositions[VertexID];
		}

		// Don't process degenerate triangles.
		if (CornerPositions[0].Equals(CornerPositions[1], UE_THRESH_POINTS_ARE_SAME) ||
			CornerPositions[0].Equals(CornerPositions[2], UE_THRESH_POINTS_ARE_SAME) ||
			CornerPositions[1].Equals(CornerPositions[2], UE_THRESH_POINTS_ARE_SAME))
		{
			WedgeIndex += 3;
			continue;
		}

		for (int32 TriVert = 0; TriVert < 3; ++TriVert, ++WedgeIndex)
		{
			const FVertexInstanceID VertexInstanceID = MeshDescription.GetTriangleVertexInstance(TriangleID, TriVert);
			const FVector3f& VertexPosition = CornerPositions[TriVert];
			UE::Private::FPrivateVertexInfo NewVert;
			NewVert.Position = CornerPositions[TriVert];
			NewVert.Tangents[0] = VertexTangents[VertexInstanceID];
			NewVert.Normal = VertexNormals[VertexInstanceID];
			NewVert.Tangents[1] = FVector3f(0.0f);
			if (!NewVert.Normal.IsNearlyZero(SMALL_NUMBER) && !NewVert.Tangents[0].IsNearlyZero(SMALL_NUMBER))
			{
				NewVert.Tangents[1] = FVector3f::CrossProduct(NewVert.Normal, NewVert.Tangents[0]).GetSafeNormal() * VertexBinormalSigns[VertexInstanceID];
			}

			// Fix bad tangents
			NewVert.Tangents[0] = NewVert.Tangents[0].ContainsNaN() ? FVector3f::ZeroVector : NewVert.Tangents[0];
			NewVert.Tangents[1] = NewVert.Tangents[1].ContainsNaN() ? FVector3f::ZeroVector : NewVert.Tangents[1];
			NewVert.Normal = NewVert.Normal.ContainsNaN() ? FVector3f::ZeroVector : NewVert.Normal;
			NewVert.Color = FLinearColor(VertexColors[VertexInstanceID]);

			for (int32 UVIndex = 0; UVIndex < NumTexCoords; UVIndex++)
			{
				if (UVIndex < VertexUVs.GetNumChannels())
				{
					NewVert.TexCoords[UVIndex] = VertexUVs.Get(VertexInstanceID, UVIndex);
				}
				else
				{
					NewVert.TexCoords[UVIndex] = FVector2f::ZeroVector;
				}
			}

			// Make sure this vertex is valid from the start
			NewVert.Validate();

			//Never add duplicated vertex instance
			//Use WedgeIndex since OverlappingCorners has been built based on that
			const TArray<int32>& DupVerts = OverlappingCorners.FindIfOverlapping(WedgeIndex);

			int32 Index = INDEX_NONE;
			for (int32 k = 0; k < DupVerts.Num(); k++)
			{
				if (DupVerts[k] >= WedgeIndex)
				{
					// the verts beyond me haven't been placed yet, so these duplicates are not relevant
					break;
				}

				int32* Location = VertsMap.Find(DupVerts[k]);
				if (Location)
				{
					UE::Private::FPrivateVertexInfo& FoundVert = Verts[*Location];

					if (NewVert.Equals(FoundVert))
					{
						Index = *Location;
						break;
					}
				}
			}
			if (Index == INDEX_NONE)
			{
				Index = Verts.Add(NewVert);
				VertsMap.Add(WedgeIndex, Index);
			}
		}
	}
	return Verts.Num();
}

void FStaticMeshOperations::ReorderMeshDescriptionPolygonGroups(const FMeshDescription& SourceMeshDescription
	, FMeshDescription& DestinationMeshDescription
	, TOptional<const FString> DestinationUnmatchMaterialName_Msg
	, TOptional<const FString> DestinationPolygonGroupCountDifferFromSource_Msg)
{
	if (SourceMeshDescription.IsEmpty() || DestinationMeshDescription.IsEmpty() || DestinationMeshDescription.PolygonGroups().Num() <= 1)
	{
		//Nothing to re-order
		return;
	}

	//Do not allow reorder if the material count is different between the destination and the source
	if (DestinationMeshDescription.PolygonGroups().Num() != SourceMeshDescription.PolygonGroups().Num())
	{
		if(DestinationPolygonGroupCountDifferFromSource_Msg.IsSet())
		{
			UE_LOGF(LogStaticMeshOperations, Warning, "%ls", *DestinationPolygonGroupCountDifferFromSource_Msg.GetValue());
		}
		return;
	}

	FStaticMeshConstAttributes SourceAttribute(SourceMeshDescription);
	TPolygonGroupAttributesConstRef<FName> SourceMaterialSlotNameAttribute = SourceAttribute.GetPolygonGroupMaterialSlotNames();
	FStaticMeshAttributes DestinationAttribute(DestinationMeshDescription);
	TPolygonGroupAttributesConstRef<FName> DestinationMaterialSlotNameAttribute = DestinationAttribute.GetPolygonGroupMaterialSlotNames();
	TMap<FPolygonGroupID, FPolygonGroupID> MatchPolygonGroupsDestSource;
	TMap<FPolygonGroupID, bool> SourceMaterialMatched;
	SourceMaterialMatched.Reserve(SourceMaterialSlotNameAttribute.GetNumElements());
	for (FPolygonGroupID SourcePolygonGroupID : SourceMeshDescription.PolygonGroups().GetElementIDs())
	{
		SourceMaterialMatched.Add(SourcePolygonGroupID, false);
	}
	TMap<FPolygonGroupID, bool> DestinationMaterialMatched;
	DestinationMaterialMatched.Reserve(DestinationMaterialSlotNameAttribute.GetNumElements());
	for (FPolygonGroupID DestinationPolygonGroupID : DestinationMeshDescription.PolygonGroups().GetElementIDs())
	{
		DestinationMaterialMatched.Add(DestinationPolygonGroupID, false);
	}

	//Find the material name match
	for (TPair<FPolygonGroupID, bool>& DestinationMatched : DestinationMaterialMatched)
	{
		const FName MaterialNameToMatch = DestinationMaterialSlotNameAttribute[DestinationMatched.Key];
		FPolygonGroupID MatchPolygonGroupID = INDEX_NONE;
		for (TPair<FPolygonGroupID, bool>& SourceMatched : SourceMaterialMatched)
		{
			if (SourceMatched.Value)
			{
				continue;
			}
			if (SourceMaterialSlotNameAttribute[SourceMatched.Key] == MaterialNameToMatch)
			{
					
				MatchPolygonGroupID = SourceMatched.Key;
				SourceMatched.Value = true;
				DestinationMatched.Value = true;
				break;
			}
		}
		if (MatchPolygonGroupID != INDEX_NONE)
		{
			MatchPolygonGroupsDestSource.FindOrAdd(DestinationMatched.Key) = MatchPolygonGroupID;
		}
	}

	if (MatchPolygonGroupsDestSource.Num() < DestinationMaterialMatched.Num() && DestinationUnmatchMaterialName_Msg.IsSet())
	{
		UE_LOGF(LogStaticMeshOperations, Warning, "%ls", *DestinationUnmatchMaterialName_Msg.GetValue());
	}

	//Iterate the unmatched destination and use the first unmatched source
	for (TPair<FPolygonGroupID, bool>& DestinationMatched : DestinationMaterialMatched)
	{
		if (DestinationMatched.Value)
		{
			//Skip this destination because its already matched
			continue;
		}

		//Match the first unmatched source we found
		for (TPair<FPolygonGroupID, bool>& SourceMatched : SourceMaterialMatched)
		{
			if (SourceMatched.Value)
			{
				//Skip this source because its already matched
				continue;
			}
			//Force match
			MatchPolygonGroupsDestSource.FindOrAdd(DestinationMatched.Key) = SourceMatched.Key;
			DestinationMatched.Value = true;
			SourceMatched.Value = true;
			break;
		}
	}

	//Since both source and destination have the same amount of material, the MatchPolygonGroupsDestSource should have the same count.
	if(ensure(MatchPolygonGroupsDestSource.Num() == DestinationMaterialMatched.Num()))
	{
		//Remap the polygon group with the correct ID
		DestinationMeshDescription.RemapPolygonGroups(MatchPolygonGroupsDestSource);
	}
}

bool FStaticMeshOperations::ValidateAndFixData(FMeshDescription& MeshDescription, const FString& DebugName)
{
	bool bHasInvalidPositions = false;
	bool bHasInvalidTangentSpaces = false;
	bool bHasInvalidUVs = false;
	bool bHasInvalidVertexColors = false;

	FStaticMeshAttributes Attributes(MeshDescription);

	TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
	for (const FVertexID VertexID : MeshDescription.Vertices().GetElementIDs())
	{
		if (VertexPositions[VertexID].ContainsNaN())
		{
			bHasInvalidPositions = true;
			VertexPositions[VertexID] = FVector3f::ZeroVector;
		}
	}
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
	TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
	const int32 NumUVs = VertexInstanceUVs.GetNumChannels();
	for (const FVertexInstanceID VertexInstanceID : MeshDescription.VertexInstances().GetElementIDs())
	{
		if (VertexInstanceNormals[VertexInstanceID].ContainsNaN())
		{
			bHasInvalidTangentSpaces = true;
			VertexInstanceNormals[VertexInstanceID] = FVector3f::Zero();
		}
		if (VertexInstanceTangents[VertexInstanceID].ContainsNaN())
		{
			bHasInvalidTangentSpaces = true;
			VertexInstanceTangents[VertexInstanceID] = FVector3f::Zero();
		}
		if (FMath::IsNaN(VertexInstanceBinormalSigns[VertexInstanceID]))
		{
			bHasInvalidTangentSpaces = true;
			VertexInstanceBinormalSigns[VertexInstanceID] = 0.0f;
		}

		for (int32 UVIndex = 0; UVIndex < NumUVs; UVIndex++)
		{
			if (VertexInstanceUVs.Get(VertexInstanceID, UVIndex).ContainsNaN())
			{
				bHasInvalidUVs = true;
				VertexInstanceUVs.Set(VertexInstanceID, UVIndex, FVector2f::Zero());
			}
		}
		if (VertexInstanceColors[VertexInstanceID].ContainsNaN())
		{
			bHasInvalidVertexColors = true;
			VertexInstanceColors[VertexInstanceID] = FVector4f::One();
		}
	}

	if (!DebugName.IsEmpty())
	{
		if (bHasInvalidPositions)
		{
			UE_LOGF(LogStaticMeshOperations, Display, "Mesh %ls has NaNs in it's vertex positions! Offending positions are set to zero.", *DebugName);
		}
		if (bHasInvalidTangentSpaces)
		{
			UE_LOGF(LogStaticMeshOperations, Display, "Mesh %ls has NaNs in it's vertex instance tangent space! Offending tangents are set to zero.", *DebugName);
		}
		if (bHasInvalidUVs)
		{
			UE_LOGF(LogStaticMeshOperations, Display, "Mesh %ls has NaNs in it's vertex instance uvs! Offending uvs are set to zero.", *DebugName);
		}
		if (bHasInvalidVertexColors)
		{
			UE_LOGF(LogStaticMeshOperations, Display, "Mesh %ls has NaNs in it's vertex instance colors! Offending colors are set to white.", *DebugName);
		}
	}

	return !bHasInvalidPositions && !bHasInvalidTangentSpaces && !bHasInvalidUVs && !bHasInvalidVertexColors;
}

float FStaticMeshOperations::ComputeMeshArea(const FMeshDescription& InMeshDescription)
{
	float AreaSum = 0.0f;

	for (FTriangleID TriangleID: InMeshDescription.Triangles().GetElementIDs())
	{
		TArrayView<const FVertexID> TriangleVertices = InMeshDescription.GetTriangleVertices(TriangleID);
		const FVector3f& PointA = InMeshDescription.GetVertexPosition(TriangleVertices[0]);
		const FVector3f& PointB = InMeshDescription.GetVertexPosition(TriangleVertices[1]);
		const FVector3f& PointC = InMeshDescription.GetVertexPosition(TriangleVertices[2]);

		AreaSum += TriangleUtilities::ComputeTriangleArea(PointA, PointB, PointC);
	}

	return AreaSum;
}

#undef LOCTEXT_NAMESPACE
