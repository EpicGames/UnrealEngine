// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "MeshAttributeArray.h"
#include "MeshAttributes.h"
#include "MeshDescription.h"
#include "UObject/NameTypes.h"


namespace MeshAttribute
{
	namespace Vertex
	{
	}

	namespace VertexInstance
	{
		extern STATICMESHDESCRIPTION_API const FName TextureCoordinate;
		extern STATICMESHDESCRIPTION_API const FName Normal;
		extern STATICMESHDESCRIPTION_API const FName Tangent;
		extern STATICMESHDESCRIPTION_API const FName BinormalSign;
		extern STATICMESHDESCRIPTION_API const FName Color;
	}

	namespace Edge
	{
		extern STATICMESHDESCRIPTION_API const FName IsHard;
	}

	namespace Triangle
	{
		extern STATICMESHDESCRIPTION_API const FName Normal;
		extern STATICMESHDESCRIPTION_API const FName Tangent;
		extern STATICMESHDESCRIPTION_API const FName Binormal;
	}

	namespace Polygon
	{
		extern STATICMESHDESCRIPTION_API const FName ObjectName;
	}

	namespace PolygonGroup
	{
		extern STATICMESHDESCRIPTION_API const FName ImportedMaterialSlotName;
	}
}


class FStaticMeshAttributes : public FMeshAttributes
{
public:

	explicit FStaticMeshAttributes(FMeshDescription& InMeshDescription)
		: FMeshAttributes(InMeshDescription)
	{}

	STATICMESHDESCRIPTION_API virtual void Register(bool bKeepExistingAttribute = false) override;
	
	static bool IsReservedAttributeName(const FName InAttributeName)
	{
		return FMeshAttributes::IsReservedAttributeName(InAttributeName) ||
               InAttributeName == MeshAttribute::VertexInstance::TextureCoordinate ||
               InAttributeName == MeshAttribute::VertexInstance::Normal ||
               InAttributeName == MeshAttribute::VertexInstance::Tangent ||
               InAttributeName == MeshAttribute::VertexInstance::BinormalSign ||
               InAttributeName == MeshAttribute::VertexInstance::Color ||
               InAttributeName == MeshAttribute::Edge::IsHard ||
               InAttributeName == MeshAttribute::Triangle::Normal ||
               InAttributeName == MeshAttribute::Triangle::Tangent ||
               InAttributeName == MeshAttribute::Triangle::Binormal ||
               InAttributeName == MeshAttribute::Polygon::ObjectName ||
               InAttributeName == MeshAttribute::PolygonGroup::ImportedMaterialSlotName
		;
	}	

	STATICMESHDESCRIPTION_API void RegisterTriangleNormalAndTangentAttributes();
	STATICMESHDESCRIPTION_API void UnregisterTriangleNormalAndTangentAttributes();

	/** Registers a mesh attribute that is used to store the name of the object that contributed to each polygon. Used to partition the mesh
	 *  into sub-objects, if needed.
	 *  See GetPolygonObjectNames.
	 */
	STATICMESHDESCRIPTION_API void RegisterPolygonObjectNameAttribute();

	TVertexInstanceAttributesRef<FVector2f> GetVertexInstanceUVs() { return MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate); }
	TVertexInstanceAttributesConstRef<FVector2f> GetVertexInstanceUVs() const { return MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate); }

	TVertexInstanceAttributesRef<FVector3f> GetVertexInstanceNormals() { return MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Normal); }
	TVertexInstanceAttributesConstRef<FVector3f> GetVertexInstanceNormals() const { return MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Normal); }

	TVertexInstanceAttributesRef<FVector3f> GetVertexInstanceTangents() { return MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Tangent); }
	TVertexInstanceAttributesConstRef<FVector3f> GetVertexInstanceTangents() const { return MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Tangent); }

	TVertexInstanceAttributesRef<float> GetVertexInstanceBinormalSigns() { return MeshDescription.VertexInstanceAttributes().GetAttributesRef<float>(MeshAttribute::VertexInstance::BinormalSign); }
	TVertexInstanceAttributesConstRef<float> GetVertexInstanceBinormalSigns() const { return MeshDescription.VertexInstanceAttributes().GetAttributesRef<float>(MeshAttribute::VertexInstance::BinormalSign); }

	TVertexInstanceAttributesRef<FVector4f> GetVertexInstanceColors() { return MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector4f>(MeshAttribute::VertexInstance::Color); }
	TVertexInstanceAttributesConstRef<FVector4f> GetVertexInstanceColors() const { return MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector4f>(MeshAttribute::VertexInstance::Color); }

	TEdgeAttributesRef<bool> GetEdgeHardnesses() { return MeshDescription.EdgeAttributes().GetAttributesRef<bool>(MeshAttribute::Edge::IsHard); }
	TEdgeAttributesConstRef<bool> GetEdgeHardnesses() const { return MeshDescription.EdgeAttributes().GetAttributesRef<bool>(MeshAttribute::Edge::IsHard); }

	TTriangleAttributesRef<FVector3f> GetTriangleNormals() { return MeshDescription.TriangleAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Triangle::Normal); }
	TTriangleAttributesConstRef<FVector3f> GetTriangleNormals() const { return MeshDescription.TriangleAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Triangle::Normal); }

	TTriangleAttributesRef<FVector3f> GetTriangleTangents() { return MeshDescription.TriangleAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Triangle::Tangent); }
	TTriangleAttributesConstRef<FVector3f> GetTriangleTangents() const { return MeshDescription.TriangleAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Triangle::Tangent); }

	TTriangleAttributesRef<FVector3f> GetTriangleBinormals() { return MeshDescription.TriangleAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Triangle::Binormal); }
	TTriangleAttributesConstRef<FVector3f> GetTriangleBinormals() const { return MeshDescription.TriangleAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Triangle::Binormal); }

	/** This attribute stores the name of the object that this part of the mesh was generated from, e.g. from a sub-object during import or during mesh merging. */
	TPolygonAttributesRef<FName> GetPolygonObjectNames() { return MeshDescription.PolygonAttributes().GetAttributesRef<FName>(MeshAttribute::Polygon::ObjectName); }
	TPolygonAttributesConstRef<FName> GetPolygonObjectNames() const { return MeshDescription.PolygonAttributes().GetAttributesRef<FName>(MeshAttribute::Polygon::ObjectName); }
	
	TPolygonGroupAttributesRef<FName> GetPolygonGroupMaterialSlotNames() { return MeshDescription.PolygonGroupAttributes().GetAttributesRef<FName>(MeshAttribute::PolygonGroup::ImportedMaterialSlotName); }
	TPolygonGroupAttributesConstRef<FName> GetPolygonGroupMaterialSlotNames() const { return MeshDescription.PolygonGroupAttributes().GetAttributesRef<FName>(MeshAttribute::PolygonGroup::ImportedMaterialSlotName); }
};


class FStaticMeshConstAttributes : public FMeshConstAttributes
{
public:

	explicit FStaticMeshConstAttributes(const FMeshDescription& InMeshDescription)
		: FMeshConstAttributes(InMeshDescription)
	{}

	TVertexInstanceAttributesConstRef<FVector2f> GetVertexInstanceUVs() const { return MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector2f>(MeshAttribute::VertexInstance::TextureCoordinate); }
	TVertexInstanceAttributesConstRef<FVector3f> GetVertexInstanceNormals() const { return MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Normal); }
	TVertexInstanceAttributesConstRef<FVector3f> GetVertexInstanceTangents() const { return MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Tangent); }
	TVertexInstanceAttributesConstRef<float> GetVertexInstanceBinormalSigns() const { return MeshDescription.VertexInstanceAttributes().GetAttributesRef<float>(MeshAttribute::VertexInstance::BinormalSign); }
	TVertexInstanceAttributesConstRef<FVector4f> GetVertexInstanceColors() const { return MeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector4f>(MeshAttribute::VertexInstance::Color); }
	TEdgeAttributesConstRef<bool> GetEdgeHardnesses() const { return MeshDescription.EdgeAttributes().GetAttributesRef<bool>(MeshAttribute::Edge::IsHard); }
	TTriangleAttributesConstRef<FVector3f> GetTriangleNormals() const { return MeshDescription.TriangleAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Triangle::Normal); }
	TTriangleAttributesConstRef<FVector3f> GetTriangleTangents() const { return MeshDescription.TriangleAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Triangle::Tangent); }
	TTriangleAttributesConstRef<FVector3f> GetTriangleBinormals() const { return MeshDescription.TriangleAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Triangle::Binormal); }
	TPolygonAttributesConstRef<FName> GetPolygonObjectNames() const { return MeshDescription.PolygonAttributes().GetAttributesRef<FName>(MeshAttribute::Polygon::ObjectName); }
	TPolygonGroupAttributesConstRef<FName> GetPolygonGroupMaterialSlotNames() const { return MeshDescription.PolygonGroupAttributes().GetAttributesRef<FName>(MeshAttribute::PolygonGroup::ImportedMaterialSlotName); }
};
