// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "MeshDescription.h"
#include "MeshDescriptionBase.h"
#include "MeshTypes.h"
#include "StaticMeshAttributes.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "StaticMeshDescription.generated.h"

class UMaterial;
class UObject;
struct FFrame;

/**
* A wrapper for MeshDescription, customized for static meshes
*/
UCLASS(BlueprintType, MinimalAPI)
class UStaticMeshDescription : public UMeshDescriptionBase
{
public:
	GENERATED_BODY()

	/** Register attributes required by static mesh description */
	STATICMESHDESCRIPTION_API virtual void RegisterAttributes() override;

	virtual FStaticMeshAttributes& GetRequiredAttributes() override
	{ 
		return static_cast<FStaticMeshAttributes&>(*RequiredAttributes);
	}

	virtual const FStaticMeshAttributes& GetRequiredAttributes() const override
	{
		return static_cast<const FStaticMeshAttributes&>(*RequiredAttributes);
	}

	UFUNCTION(BlueprintPure, Category="MeshDescription")
	STATICMESHDESCRIPTION_API FVector2D GetVertexInstanceUV(FVertexInstanceID VertexInstanceID, int32 UVIndex = 0) const;

	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	STATICMESHDESCRIPTION_API void SetVertexInstanceUV(FVertexInstanceID VertexInstanceID, FVector2D UV, int32 UVIndex = 0);

	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	STATICMESHDESCRIPTION_API void CreateCube(FVector Center, FVector HalfExtents, FPolygonGroupID PolygonGroup,
					FPolygonID& PolygonID_PlusX,
					FPolygonID& PolygonID_MinusX,
					FPolygonID& PolygonID_PlusY,
					FPolygonID& PolygonID_MinusY,
					FPolygonID& PolygonID_PlusZ,
					FPolygonID& PolygonID_MinusZ);

	UFUNCTION(BlueprintCallable, Category="MeshDescription")
	STATICMESHDESCRIPTION_API void SetPolygonGroupMaterialSlotName(FPolygonGroupID PolygonGroupID, const FName& SlotName);

public:

	TVertexInstanceAttributesRef<FVector2f> GetVertexInstanceUVs() { return GetRequiredAttributes().GetVertexInstanceUVs(); }
	TVertexInstanceAttributesConstRef<FVector2f> GetVertexInstanceUVs() const { return GetRequiredAttributes().GetVertexInstanceUVs(); }

	TVertexInstanceAttributesRef<FVector3f> GetVertexInstanceNormals() { return GetRequiredAttributes().GetVertexInstanceNormals(); }
	TVertexInstanceAttributesConstRef<FVector3f> GetVertexInstanceNormals() const { return GetRequiredAttributes().GetVertexInstanceNormals(); }

	TVertexInstanceAttributesRef<FVector3f> GetVertexInstanceTangents() { return GetRequiredAttributes().GetVertexInstanceTangents(); }
	TVertexInstanceAttributesConstRef<FVector3f> GetVertexInstanceTangents() const { return GetRequiredAttributes().GetVertexInstanceTangents(); }

	TVertexInstanceAttributesRef<float> GetVertexInstanceBinormalSigns() { return GetRequiredAttributes().GetVertexInstanceBinormalSigns(); }
	TVertexInstanceAttributesConstRef<float> GetVertexInstanceBinormalSigns() const { return GetRequiredAttributes().GetVertexInstanceBinormalSigns(); }

	TVertexInstanceAttributesRef<FVector4f> GetVertexInstanceColors() { return GetRequiredAttributes().GetVertexInstanceColors(); }
	TVertexInstanceAttributesConstRef<FVector4f> GetVertexInstanceColors() const { return GetRequiredAttributes().GetVertexInstanceColors(); }

	TEdgeAttributesRef<bool> GetEdgeHardnesses() { return GetRequiredAttributes().GetEdgeHardnesses(); }
	TEdgeAttributesConstRef<bool> GetEdgeHardnesses() const { return GetRequiredAttributes().GetEdgeHardnesses(); }

	TPolygonGroupAttributesRef<FName> GetPolygonGroupMaterialSlotNames() { return GetRequiredAttributes().GetPolygonGroupMaterialSlotNames(); }
	TPolygonGroupAttributesConstRef<FName> GetPolygonGroupMaterialSlotNames() const { return GetRequiredAttributes().GetPolygonGroupMaterialSlotNames(); }
};
