// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Logging/LogMacros.h"
#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "MeshTypes.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Optional.h"
#include "Misc/SecureHash.h"

class FName;
struct FMeshDescription;
struct FOverlappingCorners;
struct FPolygonGroupID;
struct FRawMesh;
struct FUVMapParameters;

enum class ELightmapUVVersion : int32;

typedef TMap<FPolygonGroupID, FPolygonGroupID> PolygonGroupMap;

DECLARE_LOG_CATEGORY_EXTERN(LogStaticMeshOperations, Log, All);

DECLARE_DELEGATE_ThreeParams(FAppendPolygonGroupsDelegate, const FMeshDescription& /*SourceMesh*/, FMeshDescription& /*TargetMesh*/, PolygonGroupMap& /*RemapPolygonGroup*/)

enum class EComputeNTBsFlags : uint32
{
	None = 0x00000000,	// No flags
	Normals = 0x00000001, //Force-recompute the normals and implicitly force the recomputing of tangents.
	Tangents = 0x00000002, //Force-recompute the tangents.
	UseMikkTSpace = 0x00000004, //Used when force-recomputing the tangents, use MikkTSpace.
	WeightedNTBs = 0x00000008, //Use weight surface area and angle when computing NTBs to proportionally distribute the vertex instance contribution to the normal/tangent/binormal in a smooth group.    i.e. Weight solve the cylinder problem
	BlendOverlappingNormals = 0x00000010,
	IgnoreDegenerateTriangles = 0x00000020,
};
ENUM_CLASS_FLAGS(EComputeNTBsFlags);

class FStaticMeshOperations
{
public:
	struct FAppendSettings
	{
		enum
		{
			MAX_NUM_UV_CHANNELS = 8,
		};

		FAppendPolygonGroupsDelegate PolygonGroupsDelegate;
		bool bMergeVertexColor = true;
		bool bMergeUVChannels[MAX_NUM_UV_CHANNELS] = {true};
		bool bMergeObjectName = false;
		FVector MergedAssetPivot = FVector::ZeroVector;
		TOptional<FTransform> MeshTransform; // Apply a transformation on source mesh (see MeshTransform)
	};

	/** Set the triangle tangent, normal, binormal and triangleCenter for all triangles in the mesh description. */
	static STATICMESHDESCRIPTION_API void ComputeTriangleTangentsAndNormals(FMeshDescription& MeshDescription, float ComparisonThreshold = 0.0f, const TCHAR* DebugName = nullptr);

	/** 
	 * Recompute any invalid normal, tangent or Bi-Normal for every vertex in the mesh description with the given options.
	 * If the EComputeNTBsFlags Normals or Tangents are set, the corresponding data will be force-recomputed.
	 */
	static STATICMESHDESCRIPTION_API void ComputeTangentsAndNormals(FMeshDescription& MeshDescription, EComputeNTBsFlags ComputeNTBsOptions);

	/*
	 * Make sure all normals and tangents are valid. If not, recompute them.
	 */
	static STATICMESHDESCRIPTION_API void RecomputeNormalsAndTangentsIfNeeded(FMeshDescription& MeshDescription, EComputeNTBsFlags ComputeNTBsOptions);

	/** Compute tangent and Bi-Normal using mikkt space for every vertex in the mesh description. */
	static STATICMESHDESCRIPTION_API void ComputeMikktTangents(FMeshDescription& MeshDescription, bool bIgnoreDegenerateTriangles);

	/** Determine the edge hardnesses from existing normals */
	static STATICMESHDESCRIPTION_API void DetermineEdgeHardnessesFromVertexInstanceNormals(FMeshDescription& MeshDescription, float Tolerance = UE_KINDA_SMALL_NUMBER);

	/** Convert this mesh description into the old FRawMesh format. */
	static STATICMESHDESCRIPTION_API void ConvertToRawMesh(const FMeshDescription& SourceMeshDescription, FRawMesh& DestinationRawMesh, const TMap<FName, int32>& MaterialMap);

	/** Convert old FRawMesh format to MeshDescription. */
	static STATICMESHDESCRIPTION_API void ConvertFromRawMesh(const FRawMesh& SourceRawMesh, FMeshDescription& DestinationMeshDescription, const TMap<int32, FName>& MaterialMap, bool bSkipNormalsAndTangents = false, const TCHAR* DebugName = nullptr);

	static STATICMESHDESCRIPTION_API void AppendMeshDescription(const FMeshDescription& SourceMesh, FMeshDescription& TargetMesh, const FAppendSettings& AppendSettings);
	static STATICMESHDESCRIPTION_API void AppendMeshDescriptions(const TArray<const FMeshDescription*>& SourceMeshes, FMeshDescription& TargetMesh, const FAppendSettings& AppendSettings);

	/** Check the mesh description's vertex instance normals and tangents to see if any of the values are invalid (e.g. nearly zero or they contain a NaN).
	 *  \note This does not check if the vectors are normalized (which they should for correct normals and tangents).
	 *  \param MeshDescription The mesh description object to check.
	 *  \param bHasInvalidNormals The vertex instance normal attribute contains a near zero-length vector or a NaN component value.
	 *  \param bHasInvalidTangents The vertex instance tangent attribute contains a near zero-length vector or a NaN component value.
	 */
	static STATICMESHDESCRIPTION_API void HasInvalidVertexInstanceNormalsOrTangents(const FMeshDescription& MeshDescription, bool& bHasInvalidNormals, bool& bHasInvalidTangents);

	UE_DEPRECATED(5.5, "Use HasInvalidVertexInstanceNormalsOrTangents")
	static void AreNormalsAndTangentsValid(const FMeshDescription& MeshDescription, bool& bHasInvalidNormals, bool& bHasInvalidTangents)
	{
		HasInvalidVertexInstanceNormalsOrTangents(MeshDescription, bHasInvalidNormals, bHasInvalidTangents);
	}


	/** Find all overlapping vertex using the threshold in the mesh description. */
	static STATICMESHDESCRIPTION_API void FindOverlappingCorners(FOverlappingCorners& OverlappingCorners, const FMeshDescription& MeshDescription, float ComparisonThreshold);

	/** Find all charts in the mesh description. */
	static STATICMESHDESCRIPTION_API int32 GetUVChartCount(FMeshDescription& MeshDescription, int32 SrcLightmapIndex, ELightmapUVVersion LightmapUVVersion, const FOverlappingCorners& OverlappingCorners);

	/**
	 * Find and pack UV charts for lightmap.
	 * The packing algorithm uses a rasterization method, hence the resolution parameter.
	 *
	 * If the given minimum resolution is not enough to handle all the charts, generation will fail.
	 *
	 * @param MeshDescription        Edited mesh
	 * @param SrcLightmapIndex       index of the source UV channel
	 * @param DstLightmapIndex       index of the destination UV channel
	 * @param MinLightmapResolution  Minimum resolution used for the packing
	 * @param LightmapUVVersion      Algorithm version
	 * @param OverlappingCorners     Overlapping corners of the given mesh
	 * @return                       UV layout correctly generated
	 */
	static STATICMESHDESCRIPTION_API bool CreateLightMapUVLayout(FMeshDescription& MeshDescription,
		int32 SrcLightmapIndex,
		int32 DstLightmapIndex,
		int32 MinLightmapResolution,
		ELightmapUVVersion LightmapUVVersion,
		const FOverlappingCorners& OverlappingCorners);

	/** Create some UVs from the specified mesh description data. */
	UE_DEPRECATED(5.4, "Please use GenerateUV() instead.")
	static STATICMESHDESCRIPTION_API bool GenerateUniqueUVsForStaticMesh(const FMeshDescription& MeshDescription, int32 TextureResolution, bool bMergeIdenticalMaterials, TArray<FVector2D>& OutTexCoords);
		
	enum class EGenerateUVMethod
	{
		Default,
		Legacy,
		UVAtlas,
		XAtlas,		
		PatchBuilder
	};

	struct FGenerateUVOptions
	{
		// Expected texture resolution
		int32 TextureResolution = 512;

		// Wether to fold triangles sharing the same UVs & vertex colors in the generated UV mapping.
		bool bMergeTrianglesWithIdenticalAttributes = false;

		// Method to use when generating UVs
		EGenerateUVMethod UVMethod = EGenerateUVMethod::Default;
	};

	/** Generate UV coordinates from the specified mesh description data. */
	static STATICMESHDESCRIPTION_API bool GenerateUV(const FMeshDescription& MeshDescription, const FGenerateUVOptions& Options, TArray<FVector2D>& OutTexCoords);

	/** Add a UV channel to the MeshDescription. */
	static STATICMESHDESCRIPTION_API bool AddUVChannel(FMeshDescription& MeshDescription);

	/** Insert a UV channel at the given index to the MeshDescription. */
	static STATICMESHDESCRIPTION_API bool InsertUVChannel(FMeshDescription& MeshDescription, int32 UVChannelIndex);

	/** Remove the UV channel at the given index from the MeshDescription. */
	static STATICMESHDESCRIPTION_API bool RemoveUVChannel(FMeshDescription& MeshDescription, int32 UVChannelIndex);

	/** Generate planar UV mapping for the MeshDescription */
	static STATICMESHDESCRIPTION_API void GeneratePlanarUV(const FMeshDescription& MeshDescription, const FUVMapParameters& Params, TMap<FVertexInstanceID, FVector2D>& OutTexCoords);

	/** Generate cylindrical UV mapping for the MeshDescription */
	static STATICMESHDESCRIPTION_API void GenerateCylindricalUV(FMeshDescription& MeshDescription, const FUVMapParameters& Params, TMap<FVertexInstanceID, FVector2D>& OutTexCoords);

	/** Generate box UV mapping for the MeshDescription */
	static STATICMESHDESCRIPTION_API void GenerateBoxUV(const FMeshDescription& MeshDescription, const FUVMapParameters& Params, TMap<FVertexInstanceID, FVector2D>& OutTexCoords);

	//static void RemapPolygonGroups(FMeshDescription& MeshDescription, TMap<FPolygonGroupID, FPolygonGroupID>& Remap);

	/*
	 * Move some polygon to a new PolygonGroup(section)
	 * SectionIndex: The target section we want to assign the polygon. See bRemoveEmptyPolygonGroup to know how its used
	 * TriangleIndexStart: The triangle index is compute as follow: foreach polygon {TriangleIndex += Polygon->NumberTriangles}
	 * TriangleIndexEnd: The triangle index is compute as follow: foreach polygon {TriangleIndex += Polygon->NumberTriangles}
	 * bRemoveEmptyPolygonGroup: If true, any polygonGroup that is empty after moving a polygon will be delete.
	 *                           This parameter impact how SectionIndex is use
	 *                           If param is true  : PolygonGroupTargetID.GetValue() do not necessary equal SectionIndex in case there is less sections then SectionIndex
	 *                           If param is false : PolygonGroupTargetID.GetValue() equal SectionIndex, we will add all necessary missing PolygonGroupID (this can generate empty PolygonGroupID)
	 */
	static STATICMESHDESCRIPTION_API void SwapPolygonPolygonGroup(FMeshDescription& MeshDescription, int32 SectionIndex, int32 TriangleIndexStart, int32 TriangleIndexEnd, bool bRemoveEmptyPolygonGroup);

	static STATICMESHDESCRIPTION_API void ConvertHardEdgesToSmoothGroup(const FMeshDescription& SourceMeshDescription, TArray<uint32>& FaceSmoothingMasks);

	static STATICMESHDESCRIPTION_API void ConvertSmoothGroupToHardEdges(const TArray<uint32>& FaceSmoothingMasks, FMeshDescription& DestinationMeshDescription);

	static STATICMESHDESCRIPTION_API bool HasVertexColor(const FMeshDescription& MeshDescription);

	static STATICMESHDESCRIPTION_API void BuildWeldedVertexIDRemap(const FMeshDescription& MeshDescription, const float WeldingThreshold, TMap<FVertexID, FVertexID>& OutVertexIDRemap);

	/** Computes the SHA hash of all the attributes values in the MeshDescription.
	 * @param bSkipTransientAttributes     If param is true, do not include transient attributes in the hash computation.
	 */
	static STATICMESHDESCRIPTION_API FSHAHash ComputeSHAHash(const FMeshDescription& MeshDescription, bool bSkipTransientAttributes = false);

	/** Flip the facing for a set of input polygons. */
	static STATICMESHDESCRIPTION_API void FlipPolygons(FMeshDescription& MeshDescription);

	/** 
	 * Transforms the MeshDescription data using the provided transform.
	 * @param bApplyCorrectNormalTransform Whether to correctly transform normals and tangents. Otherwise, will match the UE renderer and transform them without scale.
	 */
	static STATICMESHDESCRIPTION_API void ApplyTransform(FMeshDescription& MeshDescription, const FTransform& Transform, bool bApplyCorrectNormalTransform = false);
	static STATICMESHDESCRIPTION_API void ApplyTransform(FMeshDescription& MeshDescription, const FMatrix& Transform, bool bApplyCorrectNormalTransform = false);

	/**
	 * Return the number of unique vertices, unique vertices are the result of welding all similar vertex instances (position, UV, tangent space, color,...)
	 * 
	 */
	static STATICMESHDESCRIPTION_API int32 GetUniqueVertexCount(const FMeshDescription& MeshDescription);
	static STATICMESHDESCRIPTION_API int32 GetUniqueVertexCount(const FMeshDescription& MeshDescription, const FOverlappingCorners& OverlappingCorners);

	/**
	 * Reorder the destination mesh description polygon groups in the same order has the source mesh description polygon groups.
	 * This function use the imported material name to reorder the section.
	 * 
	 * Note: The material count MUST be the same for both the source and the destination. It will not reorder anything if the count is different.
	 */
	static STATICMESHDESCRIPTION_API void ReorderMeshDescriptionPolygonGroups(const FMeshDescription& SourceMeshDescription
		, FMeshDescription& DestinationMeshDescription
		, TOptional<const FString> UnmatchMaterialNameWarning
		, TOptional<const FString> DestinationPolygonGroupCountDifferFromSource_Msg);

	/** Verify the mesh data does not contain any NAN or INF float value, if such a case happen the value are set to zero or identity for matrix or quat. */
	static STATICMESHDESCRIPTION_API bool ValidateAndFixData(FMeshDescription& MeshDescription, const FString& DebugName);

	/** Compute the surface area of the mesh by iterating over each triangle. */
	static STATICMESHDESCRIPTION_API float ComputeMeshArea(const FMeshDescription& InMeshDescription);
};
