// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/Quat.h"
#include "UObject/ObjectMacros.h"

#include "UVMapSettings.generated.h"

/** UV map generation settings that are exposed to the user for scripting and through the editor */
USTRUCT(BlueprintType)
struct FUVMapSettings
{
	GENERATED_BODY()

	/** Length, width, height of the UV mapping gizmo */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = UVMapSettings)
	FVector Size;

	/** Tiling of the UV mapping */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = UVMapSettings)
	FVector2D UVTile;

	/** Position of the UV mapping gizmo */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GizmoTransform)
	FVector Position;

	/** Rotation of the UV mapping gizmo (angles in degrees) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GizmoTransform)
	FRotator Rotation;

	/** Scale of the UV mapping gizmo */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = GizmoTransform)
	FVector Scale;

	/** Default settings */
	FUVMapSettings()
		: Size(1.0f)
		, UVTile(1.0f, 1.0f)
		, Position(0.0f)
		, Rotation(0)
		, Scale(1.0f)
	{
	}
};

struct FUVMapParameters
{
	/** Length, width, height of the UV mapping gizmo */
	FVector Size;

	/** Tiling of the UV mapping */
	FVector2D UVTile;

	/** Position of the UV mapping gizmo */
	FVector Position;

	/** Rotation of the UV mapping gizmo (angles in degrees) */
	FQuat Rotation;

	/** Scale of the UV mapping gizmo */
	FVector Scale;

	/** Default settings */
	FUVMapParameters()
		: Size(1.0f)
		, UVTile(1.0f, 1.0f)
		, Position(0.0f)
		, Rotation(ForceInit)
		, Scale(1.0f)
	{
	}

	FUVMapParameters(const FVector& InPosition, const FQuat& InRotation, const FVector& InSize, const FVector& InScale, const FVector2D& InUVTile)
		: Size(InSize)
		, UVTile(InUVTile)
		, Position(InPosition)
		, Rotation(InRotation)
		, Scale(InScale)
	{
	}
};
