// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class StaticMeshDescription : ModuleRules
	{
		public StaticMeshDescription(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"MeshDescription"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"MeshUtilitiesCommon",
					"RawMesh",
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateIncludePathModuleNames.AddRange(
					new string[]
					{
						"GeometryProcessingInterfaces"
					}
				);

				DynamicallyLoadedModuleNames.AddRange(
					new string[]
					{
						"GeometryProcessingInterfaces"
					}
				);
			}

			AddEngineThirdPartyPrivateStaticDependencies(Target, "MikkTSpace");
		}
	}
}
